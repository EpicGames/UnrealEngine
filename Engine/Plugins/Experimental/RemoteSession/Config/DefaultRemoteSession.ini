[CoreRedirects]
+PropertyRedirects=(OldName="RemoteSessionSettings.WhitelistedChannels",NewName="AllowedChannels")
+PropertyRedirects=(OldName="RemoteSessionSettings.BlacklistedChannels",NewName="DeniedChannels")
