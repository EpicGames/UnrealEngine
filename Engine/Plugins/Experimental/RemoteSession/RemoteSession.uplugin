{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "RemoteSession",
	"Description": "A plugin for Unreal that allows one instance to act as a thin-client (rendering and input) to a second instance",
	"Category": "Experimental",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": false,
	"Installed": false,
	"EnabledByDefault": false,

	"Modules": [
		{
			"Name": "RemoteSession",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit",
			"PlatformArchitectureDenyList": [ "Win64:arm64" ],
			"PlatformDenyList": [
				"TVOS"
			]
		},
		{
			"Name": "RemoteSessionEditor",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformArchitectureDenyList": [ "Win64:arm64" ],
			"PlatformDenyList": [
				"TVOS"
			]
		}
	],
	"Plugins": [
		{
			"Name": "BackChannel",
			"Enabled": true
		},
		{
			"Name": "AppleImageUtils",
			"Enabled": true
		},
		{
			"Name": "MediaIOFramework",
			"Enabled": true
		},
		{
			"Name": "XRBase",
			"Enabled": true
		},
		{
			"Name": "PixelStreaming2",
			"Enabled": true,
			"TargetAllowList": [
				"Editor",
				"Client"
			],
			"PlatformAllowList": [
				"Win64",
				"Mac",
				"Linux"
			]
		}
	]
}
