// Copyright Epic Games, Inc. All Rights Reserved.

#include "Channels/RemoteSessionARCameraChannel.h"
#include "BackChannel/IBackChannelConnection.h"
#include "RemoteSession.h"
#include "BackChannel/Protocol/OSC/BackChannelOSCMessage.h"

#include "Async/Async.h"
#include "Channels/RemoteSessionChannel.h"
#include "Engine/Texture2D.h"
#include "SceneViewExtension.h"
#include "Materials/MaterialInterface.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Materials/MaterialRenderProxy.h"
#include "MaterialDomain.h"
#include "DataDrivenShaderPlatformInfo.h"

#include "ARTextures.h"
#include "ARBlueprintLibrary.h"

#include "IAppleImageUtilsPlugin.h"
#include "IImageWrapper.h"
#include "IImageWrapperModule.h"
#include "UObject/Package.h"

#include "MaterialShader.h"
#include "Containers/DynamicRHIResourceArray.h"
#include "CommonRenderResources.h"
#include "ScreenPass.h"
#include "SceneRenderTargetParameters.h"

#include "PostProcess/DrawRectangle.h"
#include "PostProcess/PostProcessMaterialInputs.h"
#include "RHI.h"
#include "RHIResourceUtils.h"
#include "SceneView.h"
#include "ShaderParameterMacros.h"
#include "ShaderPermutation.h"
#include "Engine/BlendableInterface.h"
#include "RenderGraph.h"
#include "Templates/SharedPointer.h"

#define CAMERA_MESSAGE_ADDRESS TEXT("/ARCamera")

TAutoConsoleVariable<int32> CVarJPEGQuality(
	TEXT("remote.arcameraquality"),
	85,
	TEXT("Sets quality (1-100)"),
	ECVF_Default);

TAutoConsoleVariable<int32> CVarJPEGColor(
	TEXT("remote.arcameracolorjpeg"),
	1,
	TEXT("1 (default) sends color data, 0 sends B&W"),
	ECVF_Default);

TAutoConsoleVariable<int32> CVarJPEGGpu(
	TEXT("remote.arcameraqcgpucompressed"),
	1,
	TEXT("1 (default) compresses on the GPU, 0 on the CPU"),
	ECVF_Default);

BEGIN_SHADER_PARAMETER_STRUCT(FRemoteSessionARCameraMaterialParameters, )
	SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, PostProcessOutput)
	SHADER_PARAMETER_STRUCT_ARRAY(FScreenPassTextureInput, PostProcessInput, [kPostProcessMaterialInputCountMax])
	SHADER_PARAMETER_STRUCT_ARRAY(FScreenPassTextureInput, PathTracingPostProcessInput, [kPathTracingPostProcessMaterialInputCountMax])
	SHADER_PARAMETER_SAMPLER(SamplerState, PostProcessInput_BilinearSampler)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<float4>, EyeAdaptationBuffer)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

class FPostProcessMaterialShader : public FMaterialShader
{
public:
	using FParameters = FRemoteSessionARCameraMaterialParameters;
	SHADER_USE_PARAMETER_STRUCT_WITH_LEGACY_BASE(FPostProcessMaterialShader, FMaterialShader);

	static bool ShouldCompilePermutation(const FMaterialShaderPermutationParameters& Parameters)
	{
		return Parameters.MaterialParameters.MaterialDomain == MD_PostProcess && !IsMobilePlatform(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("POST_PROCESS_MATERIAL"), 1);
		OutEnvironment.SetDefine(TEXT("POST_PROCESS_MATERIAL_BEFORE_TONEMAP"), (Parameters.MaterialParameters.BlendableLocation != BL_SceneColorAfterTonemapping) ? 1 : 0);

	}
};

/** Shaders to render our post process material */
class FRemoteSessionARCameraVS :
	public FPostProcessMaterialShader
{
public:
	DECLARE_SHADER_TYPE(FRemoteSessionARCameraVS, Material);

	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FPostProcessMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("POST_PROCESS_AR_PASSTHROUGH"), 1);
	}

	FRemoteSessionARCameraVS() = default;
	FRemoteSessionARCameraVS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FPostProcessMaterialShader(Initializer)
	{}

	void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FSceneView& View)
	{
		UE::Renderer::PostProcess::SetDrawRectangleParameters(BatchedParameters, this, View);
		FMaterialShader::SetViewParameters(BatchedParameters, View, View.ViewUniformBuffer);
	}
};

IMPLEMENT_SHADER_TYPE(,FRemoteSessionARCameraVS, TEXT("/Engine/Private/PostProcessMaterialShaders.usf"), TEXT("MainVS_VideoOverlay"), SF_Vertex);

class FRemoteSessionARCameraPS :
	public FPostProcessMaterialShader
{
public:
	DECLARE_SHADER_TYPE(FRemoteSessionARCameraPS, Material);

	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FPostProcessMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("OUTPUT_MOBILE_HDR"), IsMobileHDR() ? 1 : 0);
	}

	FRemoteSessionARCameraPS() = default;
	FRemoteSessionARCameraPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FPostProcessMaterialShader(Initializer)
	{}

	void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FSceneView& View, const FMaterialRenderProxy* MaterialProxy)
	{
		const FMaterial& Material = MaterialProxy->GetMaterialWithFallback(View.GetFeatureLevel(), MaterialProxy);
		FMaterialShader::SetViewParameters(BatchedParameters, View, View.ViewUniformBuffer);
		FMaterialShader::SetParameters(BatchedParameters, MaterialProxy, Material, View);
	}
};

IMPLEMENT_SHADER_TYPE(,FRemoteSessionARCameraPS, TEXT("/Engine/Private/PostProcessMaterialShaders.usf"), TEXT("MainPS_VideoOverlay"), SF_Pixel);

class FARCameraSceneViewExtension :
	public FSceneViewExtensionBase
{
public:
	FARCameraSceneViewExtension(const FAutoRegister& AutoRegister, FRemoteSessionARCameraChannel& InChannel);
	virtual ~FARCameraSceneViewExtension() { }

private:
	//~ISceneViewExtension interface
	virtual void PreRenderView_RenderThread(FRDGBuilder& GraphBuilder, FSceneView& InView) override;
	virtual void PostRenderViewFamily_RenderThread(FRDGBuilder& GraphBuilder, FSceneViewFamily& InViewFamily) override;
	virtual bool IsActiveThisFrame_Internal(const FSceneViewExtensionContext& Context) const override;
	//~ISceneViewExtension interface

	void RenderARCamera_RenderThread(FRDGBuilder& GraphBuilder, const FSceneView& InView, FRDGTextureRef ViewFamilyTexture);

	/** The channel this is rendering for */
	FRemoteSessionARCameraChannel& Channel;

	/** Note the session channel is responsible for preventing GC */
	UMaterialInterface* PPMaterial;
	/** Index buffer for drawing the quad */
	FBufferRHIRef IndexBufferRHI;
	/** Vertex buffer for drawing the quad */
	FBufferRHIRef VertexBufferRHI;
};

FARCameraSceneViewExtension::FARCameraSceneViewExtension(const FAutoRegister& AutoRegister, FRemoteSessionARCameraChannel& InChannel) :
	FSceneViewExtensionBase(AutoRegister),
	Channel(InChannel)
{
}

void FARCameraSceneViewExtension::PreRenderView_RenderThread(FRDGBuilder& GraphBuilder, FSceneView& InView)
{
	FRHICommandListBase& RHICmdList = GraphBuilder.RHICmdList;

	if (VertexBufferRHI == nullptr || !VertexBufferRHI.IsValid())
	{
		const FFilterVertex Vertices[] =
		{
			{ FVector4f(0.0f, 0.0f, 0.0f, 1.0f), FVector2f(0.0f, 0.0f) },
			{ FVector4f(1.0f, 0.0f, 0.0f, 1.0f), FVector2f(1.0f, 0.0f) },
			{ FVector4f(0.0f, 1.0f, 0.0f, 1.0f), FVector2f(0.0f, 1.0f) },
			{ FVector4f(1.0f, 1.0f, 0.0f, 1.0f), FVector2f(1.0f, 1.0f) },
		};

		VertexBufferRHI = UE::RHIResourceUtils::CreateVertexBufferFromArray(RHICmdList, TEXT("FARCameraSceneViewExtension"), EBufferUsageFlags::Static, MakeConstArrayView(Vertices));
	}

	if (IndexBufferRHI == nullptr || !IndexBufferRHI.IsValid())
	{
		// Setup index buffer
		const uint16 Indices[] = { 0, 1, 2, 2, 1, 3 };

		IndexBufferRHI = UE::RHIResourceUtils::CreateIndexBufferFromArray(RHICmdList, TEXT("FARCameraSceneViewExtension"), EBufferUsageFlags::Static, MakeConstArrayView(Indices));
	}

	PPMaterial = Channel.GetPostProcessMaterial();
}

void FARCameraSceneViewExtension::PostRenderViewFamily_RenderThread(FRDGBuilder& GraphBuilder, FSceneViewFamily& InViewFamily)
{
	if (PPMaterial == nullptr || !PPMaterial->IsValidLowLevel() ||
		VertexBufferRHI == nullptr || !VertexBufferRHI.IsValid() ||
		IndexBufferRHI == nullptr || !IndexBufferRHI.IsValid())
	{
		return;
	}

	FRDGTextureRef ViewFamilyTexture = TryCreateViewFamilyTexture(GraphBuilder, InViewFamily);
	if (!ViewFamilyTexture)
	{
		return;
	}

	for (int32 ViewIndex = 0; ViewIndex < InViewFamily.Views.Num(); ++ViewIndex)
	{
		RenderARCamera_RenderThread(GraphBuilder, *InViewFamily.Views[ViewIndex], ViewFamilyTexture);
	}
}

BEGIN_SHADER_PARAMETER_STRUCT(FRenderARCameraPassParameters, )
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneTextureUniformParameters, SceneTextures)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

void FARCameraSceneViewExtension::RenderARCamera_RenderThread(FRDGBuilder& GraphBuilder, const FSceneView& InView, FRDGTextureRef ViewFamilyTexture)
{
#if PLATFORM_DESKTOP
	if (!VertexBufferRHI || !IndexBufferRHI.IsValid())
	{
		return;
	}

	auto* PassParameters = GraphBuilder.AllocParameters<FRenderARCameraPassParameters>();
	PassParameters->RenderTargets[0] = FRenderTargetBinding(ViewFamilyTexture, ERenderTargetLoadAction::ELoad);
	check(InView.bIsViewInfo);
	PassParameters->SceneTextures = CreateSceneTextureUniformBuffer(GraphBuilder, InView, ESceneTextureSetupMode::None);

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("ARCameraOverlay"),
		PassParameters,
		ERDGPassFlags::Raster,
		[this, &InView](FRHICommandList& RHICmdList)
	{
		const auto FeatureLevel = InView.GetFeatureLevel();

		const FMaterialRenderProxy* MaterialProxy = PPMaterial->GetRenderProxy();
		const FMaterial& CameraMaterial = MaterialProxy->GetMaterialWithFallback(FeatureLevel, MaterialProxy);
		const FMaterialShaderMap* const MaterialShaderMap = CameraMaterial.GetRenderingThreadShaderMap();

		FGraphicsPipelineStateInitializer GraphicsPSOInit;
		RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);

		GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();
		GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
		GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_DepthNearOrEqual>::GetRHI();
		GraphicsPSOInit.PrimitiveType = PT_TriangleList;
		GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;

		TShaderRef<FRemoteSessionARCameraVS> VertexShader = MaterialShaderMap->GetShader<FRemoteSessionARCameraVS>();
		TShaderRef<FRemoteSessionARCameraPS> PixelShader = MaterialShaderMap->GetShader<FRemoteSessionARCameraPS>();
		GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
		GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();

		SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

		SetShaderParametersLegacyVS(RHICmdList, VertexShader, InView);
		SetShaderParametersLegacyPS(RHICmdList, PixelShader, InView, MaterialProxy);

		RHICmdList.SetStreamSource(0, VertexBufferRHI, 0);
		RHICmdList.DrawIndexedPrimitive(
			IndexBufferRHI,
			/*BaseVertexIndex=*/ 0,
			/*MinIndex=*/ 0,
			/*NumVertices=*/ 4,
			/*StartIndex=*/ 0,
			/*NumPrimitives=*/ 2,
			/*NumInstances=*/ 1
		);
	});
#endif
}

bool FARCameraSceneViewExtension::IsActiveThisFrame_Internal(const FSceneViewExtensionContext&) const
{
	return PLATFORM_DESKTOP && Channel.GetPostProcessMaterial() != nullptr;
}

static FName CameraImageParamName(TEXT("CameraImage"));

FRemoteSessionARCameraChannel::FRemoteSessionARCameraChannel(ERemoteSessionChannelMode InRole, TSharedPtr<IBackChannelConnection, ESPMode::ThreadSafe> InConnection)
	: IRemoteSessionChannel(InRole, InConnection)
	, RenderingTextureIndex(0)
	, Connection(InConnection)
	, Role(InRole)
{
	if (InRole == ERemoteSessionChannelMode::Write)
	{
		if (UARBlueprintLibrary::GetARSessionStatus().Status != EARSessionStatus::Running)
		{
			UARSessionConfig* Config = NewObject<UARSessionConfig>();
			UARBlueprintLibrary::StartARSession(Config);
		}
	}
	
	RenderingTextures[0] = nullptr;
	RenderingTextures[1] = nullptr;
	LastSetTexture = nullptr;
	PPMaterial = LoadObject<UMaterialInterface>(nullptr, TEXT("/RemoteSession/ARCameraPostProcess.ARCameraPostProcess"));
	MaterialInstanceDynamic = UMaterialInstanceDynamic::Create(PPMaterial, GetTransientPackage());
	if (MaterialInstanceDynamic != nullptr)
	{
		UTexture2D* DefaultTexture = LoadObject<UTexture2D>(nullptr, TEXT("/Engine/EngineResources/DefaultTexture.DefaultTexture"));
		// Set the initial texture so we render something until data comes in
		MaterialInstanceDynamic->SetTextureParameterValue(CameraImageParamName, DefaultTexture);
	}

	if (Role == ERemoteSessionChannelMode::Read)
	{
		// Create our image renderer
		SceneViewExtension = FSceneViewExtensions::NewExtension<FARCameraSceneViewExtension>(*this);

		auto Delegate = FBackChannelRouteDelegate::FDelegate::CreateRaw(this, &FRemoteSessionARCameraChannel::ReceiveARCameraImage);
		MessageCallbackHandle = Connection->AddRouteDelegate(CAMERA_MESSAGE_ADDRESS, Delegate);
		// #agrant todo: need equivalent
		//Connection->SetMessageOptions(CAMERA_MESSAGE_ADDRESS, 1);
	}
}

FRemoteSessionARCameraChannel::~FRemoteSessionARCameraChannel()
{
	if (Role == ERemoteSessionChannelMode::Read)
	{
		// Remove the callback so it doesn't call back on an invalid this
		Connection->RemoveRouteDelegate(CAMERA_MESSAGE_ADDRESS, MessageCallbackHandle);
	}
}

void FRemoteSessionARCameraChannel::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(RenderingTextures[0]);
	Collector.AddReferencedObject(RenderingTextures[1]);
	Collector.AddReferencedObject(PPMaterial);
	Collector.AddReferencedObject(MaterialInstanceDynamic);
}

void FRemoteSessionARCameraChannel::Tick(const float InDeltaTime)
{
	// Camera capture only works on iOS right now
#if PLATFORM_IOS
	if (Role == ERemoteSessionChannelMode::Write)
	{
		// Always send a complete compression task to make room for another
		SendARCameraImage();
		// Queue a compression task if we don't have one outstanding
		QueueARCameraImage();
	}
#endif
	if (Role == ERemoteSessionChannelMode::Read)
	{
		UpdateRenderingTexture();
		if (MaterialInstanceDynamic != nullptr)
		{
			if (UTexture2D* NextTexture = RenderingTextures[RenderingTextureIndex.GetValue()])
			{
				// Only update the material when then texture changes
				if (LastSetTexture != NextTexture)
				{
					LastSetTexture = NextTexture;
					// Update the texture to the current one
					MaterialInstanceDynamic->SetTextureParameterValue(CameraImageParamName, NextTexture);
				}
			}
		}
	}
}

void FRemoteSessionARCameraChannel::QueueARCameraImage()
{
	check(IsInGameThread());

	if (!Connection.IsValid())
	{
		return;
	}

	// Don't queue multiple compression tasks at the same time or we get a well of despair on the GPU
	if (CompressionTask.IsValid())
	{
		return;
	}

	UARTexture* CameraImage = UARBlueprintLibrary::GetARTexture(EARTextureType::CameraImage);
	if (CameraImage != nullptr)
    {
		CompressionTask = MakeShareable(new FCompressionTask());
		CompressionTask->Width = CameraImage->Size.X;
		CompressionTask->Height = CameraImage->Size.Y;
		CompressionTask->AsyncTask = IAppleImageUtilsPlugin::Get().ConvertToJPEG(CameraImage, CVarJPEGQuality.GetValueOnGameThread(), !!CVarJPEGColor.GetValueOnGameThread(), !!CVarJPEGGpu.GetValueOnGameThread());
    }
    else
    {
        UE_LOGF(LogRemoteSession, Warning, "No AR Camera Image to send!");
    }
}

void FRemoteSessionARCameraChannel::SendARCameraImage()
{
	check(IsInGameThread());

	if (!Connection.IsValid())
	{
		return;
	}

	// Bail if there's nothing to do yet
	if (!CompressionTask.IsValid() ||
		(CompressionTask.IsValid() && !CompressionTask->AsyncTask->IsDone()))
	{
		return;
	}

	if (!CompressionTask->AsyncTask->HadError())
	{
		// Copy the task so we can start GPU compressing on another one
		TSharedPtr<FCompressionTask, ESPMode::ThreadSafe> SendCompressionTask = CompressionTask;
		AsyncTask(ENamedThreads::AnyBackgroundThreadNormalTask, [this, SendCompressionTask]()
		{
			TBackChannelSharedPtr<FBackChannelOSCMessage> Msg = MakeShared<FBackChannelOSCMessage, ESPMode::ThreadSafe>(CAMERA_MESSAGE_ADDRESS);

			Msg->Write(TEXT("Width"), SendCompressionTask->Width);
			Msg->Write(TEXT("Height"), SendCompressionTask->Height);
			Msg->Write(TEXT("Data"), SendCompressionTask->AsyncTask->GetData());

			Connection->SendPacket(Msg);
		});
	}
	// Release this task so we can queue another one
	CompressionTask.Reset();
}

UMaterialInterface* FRemoteSessionARCameraChannel::GetPostProcessMaterial() const
{
	return MaterialInstanceDynamic;
}

void FRemoteSessionARCameraChannel::ReceiveARCameraImage(IBackChannelPacket& Message)
{
	IImageWrapperModule* ImageWrapperModule = FModuleManager::GetModulePtr<IImageWrapperModule>(FName("ImageWrapper"));
	if (ImageWrapperModule == nullptr)
	{
		return;
	}
	if (DecompressionTaskCount.GetValue() > 0)
	{
		// Skip if decoding is in flight so we don't have to deal with queue ordering issues
		// This will make the last one in the DecompressionQueue always be the latest image
		return;
	}
	DecompressionTaskCount.Increment();

	TSharedPtr<FDecompressedImage, ESPMode::ThreadSafe> DecompressedImage = MakeShareable(new FDecompressedImage());
	Message.Read(TEXT("Width"), DecompressedImage->Width);
	Message.Read(TEXT("Height"), DecompressedImage->Height);
	Message.Read(TEXT("Data"), DecompressedImage->ImageData);

	AsyncTask(ENamedThreads::AnyBackgroundThreadNormalTask, [this, ImageWrapperModule, DecompressedImage]
	{
		// @todo joeg -- Make FRemoteSessionFrameBufferChannel and this share code where it makes sense
		TSharedPtr<IImageWrapper> ImageWrapper = ImageWrapperModule->CreateImageWrapper(EImageFormat::JPEG);

		ImageWrapper->SetCompressed(DecompressedImage->ImageData.GetData(), DecompressedImage->ImageData.Num());

		TArray<uint8> RawData;
		if (ImageWrapper->GetRaw(ERGBFormat::BGRA, 8, RawData))
		{
			DecompressedImage->ImageData = MoveTemp(RawData);
			{
				FScopeLock sl(&DecompressionQueueLock);
				DecompressionQueue.Add(DecompressedImage);
			}
		}
		DecompressionTaskCount.Decrement();
	});
}

void FRemoteSessionARCameraChannel::UpdateRenderingTexture()
{
	TSharedPtr<FDecompressedImage, ESPMode::ThreadSafe> DecompressedImage;
	{
		FScopeLock sl(&DecompressionQueueLock);
		if (DecompressionQueue.Num())
		{
			DecompressedImage = DecompressionQueue.Last();
			DecompressionQueue.Empty();
		}
	}

	if (DecompressedImage.IsValid())
	{
		int32 NextImage = RenderingTextureIndex.GetValue() == 0 ? 1 : 0;
		// The next texture is still being updated on the rendering thread
		if (RenderingTexturesUpdateCount[NextImage].GetValue() > 0)
		{
			return;
		}
		RenderingTexturesUpdateCount[NextImage].Increment();

		// Create a texture if the sizes changed or the texture hasn't been created yet
		if (RenderingTextures[NextImage] == nullptr || DecompressedImage->Width != RenderingTextures[NextImage]->GetSizeX() || DecompressedImage->Height != RenderingTextures[NextImage]->GetSizeY())
		{
			RenderingTextures[NextImage] = UTexture2D::CreateTransient(DecompressedImage->Width, DecompressedImage->Height);
			RenderingTextures[NextImage]->SRGB = 0;
			RenderingTextures[NextImage]->UpdateResource();
		}

		// Update it on the render thread. There shouldn't (...) be any harm in GT code using it from this point
		FUpdateTextureRegion2D* Region = new FUpdateTextureRegion2D(0, 0, 0, 0, DecompressedImage->Width, DecompressedImage->Height);
		TArray<uint8>* TextureData = new TArray<uint8>(MoveTemp(DecompressedImage->ImageData));

		RenderingTextures[NextImage]->UpdateTextureRegions(0, 1, Region, 4 * DecompressedImage->Width, sizeof(FColor), TextureData->GetData(), [this, NextImage, TextureData](auto InTextureData, auto InRegions)
		{
			RenderingTextureIndex.Set(NextImage);
			RenderingTexturesUpdateCount[NextImage].Decrement();
			delete TextureData;
			delete InRegions;
		});
	}
}


TSharedPtr<IRemoteSessionChannel> FRemoteSessionARCameraChannelFactoryWorker::Construct(ERemoteSessionChannelMode InMode, TSharedPtr<IBackChannelConnection, ESPMode::ThreadSafe> InConnection) const
{
	// Client side sending only works on iOS with Android coming in the future
	bool bSessionTypeSupported = UARBlueprintLibrary::IsSessionTypeSupported(EARSessionType::World);
	bool IsSupported = (InMode == ERemoteSessionChannelMode::Read) || (PLATFORM_IOS && bSessionTypeSupported);
	if (IsSupported)
	{
		return MakeShared<FRemoteSessionARCameraChannel>(InMode, InConnection);
	}
	else
	{
		UE_LOGF(LogRemoteSession, Warning, "FRemoteSessionARCameraChannel does not support sending on this platform/device");
	}
	return TSharedPtr<IRemoteSessionChannel>();
}
