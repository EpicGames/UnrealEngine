// Copyright Epic Games, Inc. All Rights Reserved.

#include "Channels/RemoteSessionARSystemChannel.h"
#include "ARTrackable.h"
#include "Async/TaskGraphInterfaces.h"
#include "Misc/PackageName.h"
#include "RemoteSession.h"
#include "BackChannel/IBackChannelConnection.h"
#include "BackChannel/Protocol/OSC/BackChannelOSCMessage.h"
#include "Channels/RemoteSessionXRTrackingChannel.h"
#include "MessageHandler/Messages.h"
#include "ARBlueprintLibrary.h"

#include "Serialization/MemoryReader.h"
#include "Serialization/ObjectAndNameAsStringProxyArchive.h"

DECLARE_CYCLE_STAT(TEXT("ARSystemChannel_Receive"), STAT_ARSystemChannel_Receive, STATGROUP_Game);

#define INIT_MESSAGE_ADDRESS TEXT("/ARInit")
#define ADD_TRACKABLE_MESSAGE_ADDRESS TEXT("/AddTrackable")
#define UPDATE_TRACKABLE_MESSAGE_ADDRESS TEXT("/UpdateTrackable")
#define REMOVE_TRACKABLE_MESSAGE_ADDRESS TEXT("/RemoveTrackable")

TSharedPtr<FARSystemProxy, ESPMode::ThreadSafe> FARSystemProxy::FactoryInstance;

FARSystemProxy::FARSystemProxy()
	: SessionConfig(nullptr)
{
}

FARSystemProxy::~FARSystemProxy()
{
}

TSharedPtr<FARSystemProxy, ESPMode::ThreadSafe> FARSystemProxy::Get()
{
	if (!FactoryInstance.IsValid())
	{
		FactoryInstance = MakeShared<FARSystemProxy, ESPMode::ThreadSafe>();
	}
	return FactoryInstance;
}

IARSystemSupport* FARSystemProxy::GetARSystemPtr()
{
	return Get().Get();
}

void FARSystemProxy::Destroy()
{
	if (FactoryInstance.IsValid())
	{
		FactoryInstance.Reset();
	}
}

void FARSystemProxy::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(SessionConfig);
	Collector.AddReferencedObjects(TrackedGeometries);
}

/** Returns true/false based on whether AR features are available */
bool FARSystemProxy::IsARAvailable() const
{
	return true;
}

EARTrackingQuality FARSystemProxy::OnGetTrackingQuality() const
{
	// @todo JoeG - Added a delegate for tracking quality so we can send that
	return EARTrackingQuality::NotTracking;
}

EARTrackingQualityReason FARSystemProxy::OnGetTrackingQualityReason() const
{
	return EARTrackingQualityReason::InsufficientFeatures;
}

FARSessionStatus FARSystemProxy::OnGetARSessionStatus() const
{
	// @todo JoeG - Added a delegate for this so we can send that
	FARSessionStatus Status;
	Status.Status = EARSessionStatus::Other;
	return Status;
}

TArray<UARTrackedGeometry*> FARSystemProxy::OnGetAllTrackedGeometries() const
{
	check(IsInGameThread());

	TArray<UARTrackedGeometry*> Geometries;
	ObjectPtrDecay(TrackedGeometries).GenerateValueArray(Geometries);
	return Geometries;
}

bool FARSystemProxy::OnIsTrackingTypeSupported(EARSessionType SessionType) const
{
	return SessionConfig != nullptr && SessionConfig->GetSessionType() == SessionType;
}

EARWorldMappingState FARSystemProxy::OnGetWorldMappingStatus() const
{
	// @todo JoeG - Added a delegate for this so we can send that
	return EARWorldMappingState::NotAvailable;
}

TArray<FARVideoFormat> FARSystemProxy::OnGetSupportedVideoFormats(EARSessionType SessionType) const
{
	check(IsInGameThread());

	if (SessionConfig != nullptr && SessionConfig->GetSessionType() == SessionType)
	{
		return SupportedFormats;
	}
	return TArray<FARVideoFormat>();
}

void FARSystemProxy::SetSupportedVideoFormats(const TArray<FARVideoFormat>& InFormats)
{
	check(IsInGameThread());

	SupportedFormats = InFormats;
}

void FARSystemProxy::SetSessionConfig(UARSessionConfig* InConfig)
{
	check(IsInGameThread());

	SessionConfig = InConfig;
}

void FARSystemProxy::AddTrackable(UARTrackedGeometry* Added)
{
	check(IsInGameThread());

	TrackedGeometries.Add(Added->UniqueId, Added);
	TriggerOnTrackableAddedDelegates(Added);
}

UARTrackedGeometry* FARSystemProxy::GetTrackable(FGuid UniqueId)
{
	check(IsInGameThread());

	auto* GeometrySearchResult = TrackedGeometries.Find(UniqueId);
	return *GeometrySearchResult;
}

void FARSystemProxy::NotifyUpdated(UARTrackedGeometry* Updated)
{
	check(IsInGameThread());

	TriggerOnTrackableUpdatedDelegates(Updated);
}

void FARSystemProxy::RemoveTrackable(FGuid UniqueId)
{
	check(IsInGameThread());

	UARTrackedGeometry* TrackedGeometryBeingRemoved = TrackedGeometries.FindChecked(UniqueId);
	TrackedGeometryBeingRemoved->UpdateTrackingState(EARTrackingState::StoppedTracking);

	TriggerOnTrackableRemovedDelegates(TrackedGeometryBeingRemoved);

	TrackedGeometries.Remove(UniqueId);
}


FRemoteSessionARSystemChannel::FRemoteSessionARSystemChannel(ERemoteSessionChannelMode InRole, TSharedPtr<IBackChannelConnection, ESPMode::ThreadSafe> InConnection)
	: FRemoteSessionXRTrackingChannel(InRole, InConnection, InRole == ERemoteSessionChannelMode::Read ? FARSystemProxy::GetARSystemPtr() : nullptr)
{
	// Are we receiving updates from the AR system? or sending them
	if (Role == ERemoteSessionChannelMode::Read)
	{
		InitMessageCallbackHandle = Connection->AddRouteDelegate(INIT_MESSAGE_ADDRESS, FBackChannelRouteDelegate::FDelegate::CreateRaw(this, &FRemoteSessionARSystemChannel::ReceiveARInit));
		
		// #agrant todo: need equivalent
		//Connection->SetMessageOptions(INIT_MESSAGE_ADDRESS, 1);

		AddMessageCallbackHandle = Connection->AddRouteDelegate(ADD_TRACKABLE_MESSAGE_ADDRESS, FBackChannelRouteDelegate::FDelegate::CreateRaw(this, &FRemoteSessionARSystemChannel::ReceiveAddTrackable));
		// #agrant todo: need equivalent
		//Connection->SetMessageOptions(ADD_TRACKABLE_MESSAGE_ADDRESS, 1000);

		UpdateMessageCallbackHandle = Connection->AddRouteDelegate(UPDATE_TRACKABLE_MESSAGE_ADDRESS, FBackChannelRouteDelegate::FDelegate::CreateRaw(this, &FRemoteSessionARSystemChannel::ReceiveUpdateTrackable));
		// #agrant todo: need equivalent
		//Connection->SetMessageOptions(UPDATE_TRACKABLE_MESSAGE_ADDRESS, 1000);

		RemoveMessageCallbackHandle = Connection->AddRouteDelegate(REMOVE_TRACKABLE_MESSAGE_ADDRESS, FBackChannelRouteDelegate::FDelegate::CreateRaw(this, &FRemoteSessionARSystemChannel::ReceiveRemoveTrackable));
		// #agrant todo: need equivalent
		//Connection->SetMessageOptions(REMOVE_TRACKABLE_MESSAGE_ADDRESS, 1000);
	}
	else
	{
		// Add the 3 AR trackable handlers
		OnTrackableAddedDelegateHandle = UARBlueprintLibrary::AddOnTrackableAddedDelegate_Handle(FOnTrackableAddedDelegate::CreateRaw(this, &FRemoteSessionARSystemChannel::SendAddedMessage));
		OnTrackableUpdatedDelegateHandle = UARBlueprintLibrary::AddOnTrackableUpdatedDelegate_Handle(FOnTrackableAddedDelegate::CreateRaw(this, &FRemoteSessionARSystemChannel::SendUpdatedMessage));
		OnTrackableRemovedDelegateHandle = UARBlueprintLibrary::AddOnTrackableRemovedDelegate_Handle(FOnTrackableAddedDelegate::CreateRaw(this, &FRemoteSessionARSystemChannel::SendRemovedMessage));
	}
}

FRemoteSessionARSystemChannel::~FRemoteSessionARSystemChannel()
{
	if (Role == ERemoteSessionChannelMode::Read)
	{
		// Clean up all of the message handlers
		Connection->RemoveRouteDelegate(INIT_MESSAGE_ADDRESS, InitMessageCallbackHandle);
		Connection->RemoveRouteDelegate(ADD_TRACKABLE_MESSAGE_ADDRESS, AddMessageCallbackHandle);
		Connection->RemoveRouteDelegate(UPDATE_TRACKABLE_MESSAGE_ADDRESS, UpdateMessageCallbackHandle);
		Connection->RemoveRouteDelegate(REMOVE_TRACKABLE_MESSAGE_ADDRESS, RemoveMessageCallbackHandle);
	}
	else
	{
		// Remove all of the AR notifications
		UARBlueprintLibrary::ClearOnTrackableAddedDelegate_Handle(OnTrackableAddedDelegateHandle);
		UARBlueprintLibrary::ClearOnTrackableUpdatedDelegate_Handle(OnTrackableUpdatedDelegateHandle);
		UARBlueprintLibrary::ClearOnTrackableRemovedDelegate_Handle(OnTrackableRemovedDelegateHandle);
	}

	FARSystemProxy::Destroy();
}

void FRemoteSessionARSystemChannel::ReceiveARInit(IBackChannelPacket& Message)
{
	check(ARSystemSupport != nullptr);

	TArray<uint8> MsgData;
	Message.Read(TEXT("Data"), MsgData);

	FMemoryReader Ar(MsgData);
	TwoParamMsg<FString, TArray<FARVideoFormat>> MsgParam(Ar);

	UE_LOGF(LogRemoteSession, Log, "Received AR session config (%ls)", *MsgParam.Param1);

	// Since we are dealing with creating new UObjects, this needs to happen on the game thread
	auto ARConfigTask = FSimpleDelegateGraphTask::FDelegate::CreateRaw(this, &FRemoteSessionARSystemChannel::ReceiveARInit_GameThread, MsgParam.Param1, MsgParam.Param2);
	FSimpleDelegateGraphTask::CreateAndDispatchWhenReady(ARConfigTask, GET_STATID(STAT_ARSystemChannel_Receive), nullptr, ENamedThreads::GameThread);
}

void FRemoteSessionARSystemChannel::ReceiveARInit_GameThread(FString ConfigObjectPathName, TArray<FARVideoFormat> Formats)
{
	check(IsInGameThread());

	FARSystemProxy::Get()->SetSupportedVideoFormats(Formats);

	// Load and set the config object that was passed in
	checkf(!FPackageName::IsShortPackageName(ConfigObjectPathName), TEXT("ReceiveARInit_GameThread expects class path name, received: \"%s\". Check SendARInitMessage()."), *ConfigObjectPathName);
	UARSessionConfig* SessionConfig = FindObject<UARSessionConfig>(nullptr, *ConfigObjectPathName);
	if (SessionConfig == nullptr)
	{
		SessionConfig = LoadObject<UARSessionConfig>(nullptr, *ConfigObjectPathName);
	}
	// If the object could not be loaded (transient one) then create a default one and set that
	if (SessionConfig == nullptr)
	{
		SessionConfig = NewObject<UARSessionConfig>();
	}
	FARSystemProxy::Get()->SetSessionConfig(SessionConfig);
}

void FRemoteSessionARSystemChannel::ReceiveAddTrackable(IBackChannelPacket& Message)
{
	FString ClassPathName;
	Message.Read(TEXT("Name"), ClassPathName);
	TSharedPtr<TArray<uint8>, ESPMode::ThreadSafe> DataCopy = MakeShareable(new TArray<uint8>());
	Message.Read(TEXT("Data"), *DataCopy);

	// Since we are dealing with creating new UObjects, this needs to happen on the game thread
	auto AddTrackableTask = FSimpleDelegateGraphTask::FDelegate::CreateRaw(this, &FRemoteSessionARSystemChannel::ReceiveAddTrackable_GameThread, ClassPathName, DataCopy);
	FSimpleDelegateGraphTask::CreateAndDispatchWhenReady(AddTrackableTask, GET_STATID(STAT_ARSystemChannel_Receive), nullptr, ENamedThreads::GameThread);
}

void FRemoteSessionARSystemChannel::ReceiveAddTrackable_GameThread(FString ClassPathName, TSharedPtr<TArray<uint8>, ESPMode::ThreadSafe> DataCopy)
{
	UClass* TrackableClass = FindObject<UClass>(nullptr, *ClassPathName);
	// We shouldn't have to load this since these are all native, but in case some AR platform has non-native classes...
	if (TrackableClass == nullptr)
	{
		TrackableClass = LoadObject<UClass>(nullptr, *ClassPathName);
	}

	if (TrackableClass != nullptr)
	{
		UARTrackedGeometry* TrackedGeometry = NewObject<UARTrackedGeometry>(nullptr, TrackableClass);

		FMemoryReader MemoryReader(*DataCopy, true);
		FObjectAndNameAsStringProxyArchive Ar(MemoryReader, true);
		TrackedGeometry->Serialize(Ar);

		TrackedGeometry->SetLastUpdateTimestamp(FPlatformTime::Seconds());

		FARSystemProxy::Get()->AddTrackable(TrackedGeometry);

		UE_LOGF(LogRemoteSession, Log, "Added new trackable (%ls) with class (%ls)", *TrackedGeometry->GetName(), *ClassPathName);
	}
	else
	{
		UE_LOGF(LogRemoteSession, Warning, "Couldn't find class (%ls) for added trackable", *ClassPathName);
	}
}

void FRemoteSessionARSystemChannel::ReceiveUpdateTrackable(IBackChannelPacket& Message)
{
	FGuid UniqueId;
	FString StringGuid;
	Message.Read(TEXT("GUID"), StringGuid);
	FGuid::Parse(StringGuid, UniqueId);
	TSharedPtr<TArray<uint8>, ESPMode::ThreadSafe> DataCopy = MakeShareable(new TArray<uint8>());
	Message.Read(TEXT("Data"), *DataCopy);

	// Since we are dealing with updating UObjects, this needs to happen on the game thread
	auto UpdateTrackableTask = FSimpleDelegateGraphTask::FDelegate::CreateRaw(this, &FRemoteSessionARSystemChannel::ReceiveUpdateTrackable_GameThread, UniqueId, DataCopy);
	FSimpleDelegateGraphTask::CreateAndDispatchWhenReady(UpdateTrackableTask, GET_STATID(STAT_ARSystemChannel_Receive), nullptr, ENamedThreads::GameThread);
}

void FRemoteSessionARSystemChannel::ReceiveUpdateTrackable_GameThread(FGuid UniqueId, TSharedPtr<TArray<uint8>, ESPMode::ThreadSafe> DataCopy)
{
	check(IsInGameThread());

	UARTrackedGeometry* Updated = FARSystemProxy::Get()->GetTrackable(UniqueId);
	if (Updated != nullptr)
	{
		FMemoryReader MemoryReader(*DataCopy, true);
		FObjectAndNameAsStringProxyArchive Ar(MemoryReader, true);
		Updated->Serialize(Ar);

		Updated->SetLastUpdateTimestamp(FPlatformTime::Seconds());

		FARSystemProxy::Get()->NotifyUpdated(Updated);

		UE_LOGF(LogRemoteSession, Log, "Updated trackable (%ls) with UniqueId (%ls)", *Updated->GetName(), *UniqueId.ToString());
	}
	else
	{
		UE_LOGF(LogRemoteSession, Warning, "Couldn't find trackable with UniqueId (%ls)", *UniqueId.ToString());
	}
}

void FRemoteSessionARSystemChannel::ReceiveRemoveTrackable(IBackChannelPacket& Message)
{
	FGuid UniqueId;
	FString StringGuid;
	Message.Read(TEXT("GUID"), StringGuid);
	FGuid::Parse(StringGuid, UniqueId);

	// Since we are dealing with updating UObjects, this needs to happen on the game thread
	auto RemoveTrackableTask = FSimpleDelegateGraphTask::FDelegate::CreateRaw(this, &FRemoteSessionARSystemChannel::ReceiveRemoveTrackable_GameThread, UniqueId);
	FSimpleDelegateGraphTask::CreateAndDispatchWhenReady(RemoveTrackableTask, GET_STATID(STAT_ARSystemChannel_Receive), nullptr, ENamedThreads::GameThread);
}

void FRemoteSessionARSystemChannel::ReceiveRemoveTrackable_GameThread(FGuid UniqueId)
{
	check(IsInGameThread());

	FARSystemProxy::Get()->RemoveTrackable(UniqueId);

	UE_LOGF(LogRemoteSession, Log, "Removed trackable with UniqueId (%ls)", *UniqueId.ToString());
}

void FRemoteSessionARSystemChannel::SendARInitMessage()
{
	if (Connection.IsValid())
	{
		UARSessionConfig* Config = UARBlueprintLibrary::GetSessionConfig();
		if (Config != nullptr)
		{
			FString PathName = Config->GetPathName();
			TArray<FARVideoFormat> SupportedFormats = UARBlueprintLibrary::GetSupportedVideoFormats(Config->GetSessionType());

			TwoParamMsg<FString, TArray<FARVideoFormat>> MsgParam(PathName, SupportedFormats);
			TBackChannelSharedPtr<FBackChannelOSCMessage> Msg = MakeShared<FBackChannelOSCMessage, ESPMode::ThreadSafe>(INIT_MESSAGE_ADDRESS);
			Msg->Write(TEXT("Data"),MsgParam.AsData());

			Connection->SendPacket(Msg);

			UE_LOGF(LogRemoteSession, Log, "Sent AR init message with session config (%ls)", *PathName);
		}
	}
}

void FRemoteSessionARSystemChannel::SendAddedMessage(UARTrackedGeometry* Added)
{
	// Serialize the object into a buffer for sending
	SerializeBuffer.Reset();
	FMemoryWriter MemoryWriter(SerializeBuffer, true);
	FObjectAndNameAsStringProxyArchive Ar(MemoryWriter, false);
	Added->Serialize(Ar);

	FString ClassPathName = Added->GetClass()->GetPathName();

	TBackChannelSharedPtr<FBackChannelOSCMessage> Msg = MakeShared<FBackChannelOSCMessage, ESPMode::ThreadSafe>(ADD_TRACKABLE_MESSAGE_ADDRESS);

	Msg->Write(TEXT("Name"), ClassPathName);
	Msg->Write(TEXT("Data"), SerializeBuffer);
	Connection->SendPacket(Msg);

	UE_LOGF(LogRemoteSession, Log, "Sent trackable added (%ls)", *Added->GetName());
}

void FRemoteSessionARSystemChannel::SendUpdatedMessage(UARTrackedGeometry* Updated)
{
	// Serialize the object into a buffer for sending
	SerializeBuffer.Reset();
	FMemoryWriter MemoryWriter(SerializeBuffer, true);
	FObjectAndNameAsStringProxyArchive Ar(MemoryWriter, false);
	Updated->Serialize(Ar);

	FString TrackableGuid = Updated->UniqueId.ToString();

	TBackChannelSharedPtr<FBackChannelOSCMessage> Msg = MakeShared<FBackChannelOSCMessage, ESPMode::ThreadSafe>(UPDATE_TRACKABLE_MESSAGE_ADDRESS);

	Msg->Write(TEXT("GUID"), TrackableGuid);
	Msg->Write(TEXT("Data"), SerializeBuffer);
	Connection->SendPacket(Msg);

	UE_LOGF(LogRemoteSession, Log, "Sent trackable updated (%ls)", *Updated->GetName());
}

void FRemoteSessionARSystemChannel::SendRemovedMessage(UARTrackedGeometry* Removed)
{
	FString TrackableGuid = Removed->UniqueId.ToString();

	TBackChannelSharedPtr<FBackChannelOSCMessage> Msg = MakeShared<FBackChannelOSCMessage, ESPMode::ThreadSafe>(UPDATE_TRACKABLE_MESSAGE_ADDRESS);

	Msg->Write(TEXT("GUID"), TrackableGuid);
	Connection->SendPacket(Msg);

	UE_LOGF(LogRemoteSession, Log, "Sent trackable removed (%ls)", *Removed->GetName());
}

TSharedPtr<IRemoteSessionChannel> FRemoteSessionARSystemChannelFactoryWorker::Construct(ERemoteSessionChannelMode InMode, TSharedPtr<IBackChannelConnection, ESPMode::ThreadSafe> InConnection) const
{
	bool IsSupported = (InMode == ERemoteSessionChannelMode::Read) || UARBlueprintLibrary::IsSessionTypeSupported(EARSessionType::World);
	if (IsSupported)
	{
		return MakeShared<FRemoteSessionARSystemChannel>(InMode, InConnection);
	}
	else
	{
		UE_LOGF(LogRemoteSession, Warning, "FRemoteSessionARSystemChannel does not support sending on this platform/device");
	}
	return TSharedPtr<IRemoteSessionChannel>();
}
