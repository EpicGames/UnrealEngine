// Copyright Epic Games, Inc. All Rights Reserved.

#include "Channels/RemoteSessionChannel.h"
#include "RemoteSession.h"


void FRemoteSessionChannelRegistry::RegisterChannelFactory(const TCHAR* InChannelName, ERemoteSessionChannelMode InHostMode, TWeakPtr<IRemoteSessionChannelFactoryWorker> InFactory)
{
	RegisteredFactories.Add(InChannelName, InFactory);
	KnownChannels.Emplace(InChannelName, InHostMode);
}

void FRemoteSessionChannelRegistry::RemoveChannelFactory(TWeakPtr<IRemoteSessionChannelFactoryWorker> InFactory)
{
	// #agrant todo: implement this
	//check(0);
	//RegisteredFactories.Add(InChannelName, InFactory);

	for (auto& It : RegisteredFactories)
	{
		if (It.Value == InFactory)
		{
			KnownChannels = KnownChannels.FilterByPredicate([It](const FRemoteSessionChannelInfo& Item) {
				return Item.Type == It.Key;
			});

			RegisteredFactories.Remove(It.Key);
			return;
		}
	}
}


TSharedPtr<IRemoteSessionChannel> FRemoteSessionChannelRegistry::CreateChannel(const FStringView InChannelName, ERemoteSessionChannelMode InMode, TBackChannelSharedPtr<IBackChannelConnection> InConnection)
{
	FString Name = FString::ConstructFromPtrSize(InChannelName.GetData(), InChannelName.Len());

	if (!RegisteredFactories.Contains(Name))
	{
		UE_LOGF(LogRemoteSession, Error, "No factory registered for RemoteSession channel '%ls'", *Name);
		return TSharedPtr<IRemoteSessionChannel>();
	}

	TSharedPtr<IRemoteSessionChannelFactoryWorker> PinnedFactory = RegisteredFactories[Name].Pin();

	if (PinnedFactory.IsValid())
	{
		return PinnedFactory->Construct(InMode, InConnection);
	}

	return TSharedPtr<IRemoteSessionChannel>();
}

const TArray<FRemoteSessionChannelInfo>& FRemoteSessionChannelRegistry::GetRegisteredFactories() const
{
	return KnownChannels;
}
