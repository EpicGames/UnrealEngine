// Copyright Epic Games, Inc. All Rights Reserved.

#include "Channels/RemoteSessionFrameBufferChannel.h"
#include "Channels/RemoteSessionChannel.h"
#include "ImageProviders/RemoteSessionFrameBufferImageProvider.h"

#include "RemoteSessionUtils.h"


// FRemoteSessionFrameBufferChannel is deprecated. Please use FRemoteSessionImageChannel.
// FRemoteSessionFrameBufferChannelFactoryWorker was created for backward compatibility with older app

class FRemoteSessionFrameBufferChannelFactoryWorker : public IRemoteSessionChannelFactoryWorker
{
public:
	TSharedPtr<IRemoteSessionChannel> Construct(ERemoteSessionChannelMode InMode, TSharedPtr<IBackChannelConnection, ESPMode::ThreadSafe> InConnection) const
	{
		TSharedPtr<FRemoteSessionImageChannel> Channel = MakeShared<FRemoteSessionImageChannel>(InMode, InConnection);

		if (InMode == ERemoteSessionChannelMode::Write)
		{
			TSharedPtr<FRemoteSessionFrameBufferImageProvider> ImageProvider = MakeShared<FRemoteSessionFrameBufferImageProvider>(Channel->GetImageSender());

			{
				const URemoteSessionSettings* Settings = URemoteSessionSettings::StaticClass()->GetDefaultObject<URemoteSessionSettings>();

				ImageProvider->SetCaptureFrameRate(Settings->FrameRate);
				Channel->SetCompressQuality(Settings->ImageQuality);
			}

			{
				TWeakPtr<SWindow> InputWindow;
				TWeakPtr<FSceneViewport> SceneViewport;
				FRemoteSessionUtils::FindSceneViewport(InputWindow, SceneViewport);

				if (TSharedPtr<FSceneViewport> SceneViewPortPinned = SceneViewport.Pin())
				{
					ImageProvider->SetCaptureViewport(SceneViewPortPinned.ToSharedRef());
				}
			}

			Channel->SetImageProvider(ImageProvider);
		}
		return Channel;
	}
};

REGISTER_CHANNEL_FACTORY(FRemoteSessionFrameBufferChannel, FRemoteSessionFrameBufferChannelFactoryWorker, ERemoteSessionChannelMode::Write);
