// Copyright Epic Games, Inc. All Rights Reserved.

#include "Channels/RemoteSessionImageChannel.h"
#include "BackChannel/IBackChannelConnection.h"
#include "RHITypes.h"
#include "RemoteSession.h"
#include "BackChannel/Protocol/OSC/BackChannelOSCMessage.h"
#include "Channels/RemoteSessionChannel.h"
#include "HAL/Event.h"
#include "IImageWrapper.h"
#include "HAL/RunnableThread.h"
#include "IImageWrapperModule.h"
#include "Engine/Texture2D.h"
#include "Modules/ModuleManager.h"
#include "ImageProviders/RemoteSessionFrameBufferImageProvider.h"
#include "RemoteSessionUtils.h"
#include "Stats/StatsTrace.h"



static int32 QualityMainSetting = 0;
static FAutoConsoleVariableRef CVarQualityOverride(
	TEXT("remote.quality"), QualityMainSetting,
	TEXT("Sets quality (1-100)"),
	ECVF_Default);

class FRemoteSessionImageChannelFactoryWorker : public IRemoteSessionChannelFactoryWorker
{
public:
	TSharedPtr<IRemoteSessionChannel> Construct(ERemoteSessionChannelMode InMode, TBackChannelSharedPtr<IBackChannelConnection> InConnection) const
	{
		return MakeShared<FRemoteSessionImageChannel>(InMode, InConnection);
	}
};

REGISTER_CHANNEL_FACTORY(FRemoteSessionImageChannel, FRemoteSessionImageChannelFactoryWorker, ERemoteSessionChannelMode::Write);


FRemoteSessionImageChannel::FImageSender::FImageSender(TSharedPtr<IBackChannelConnection, ESPMode::ThreadSafe> InConnection)
{
	Connection = InConnection;
	CompressQuality = 0;
	NumSentImages = 0;
}

void FRemoteSessionImageChannel::FImageSender::SetCompressQuality(int32 InQuality)
{
	CompressQuality = InQuality;
}

void FRemoteSessionImageChannel::FImageSender::SendRawImageToClients(int32 Width, int32 Height, const TArray<FColor>& ImageData, int32 BytesPerRow)
{
	SendRawImageToClients(Width, Height, ImageData.GetData(), ImageData.GetAllocatedSize(), BytesPerRow);
}

void FRemoteSessionImageChannel::FImageSender::SendRawImageToClients(int32 Width, int32 Height, const void* ImageData, int32 AllocatedImageDataSize, int32 BytesPerRow)
{
	static bool SkipImages = FParse::Param(FCommandLine::Get(), TEXT("remote.noimage"));

	// Can be released on the main thread at anytime so hold onto it
	TSharedPtr<IBackChannelConnection, ESPMode::ThreadSafe> LocalConnection = Connection.Pin();

	if (LocalConnection.IsValid() && SkipImages == false)
	{
		const double TimeNow = FPlatformTime::Seconds();

		// created on demand because there can be multiple SendImage requests in flight
		IImageWrapperModule* ImageWrapperModule = FModuleManager::GetModulePtr<IImageWrapperModule>(FName("ImageWrapper"));
		if (ImageWrapperModule != nullptr)
		{
			TSharedPtr<IImageWrapper> ImageWrapper = ImageWrapperModule->CreateImageWrapper(EImageFormat::JPEG);

			ImageWrapper->SetRaw(ImageData, AllocatedImageDataSize, Width, Height, ERGBFormat::BGRA, 8, BytesPerRow);

			const int32 CurrentQuality = QualityMainSetting > 0 ? QualityMainSetting : CompressQuality.Load();
			const TArray64<uint8> JPGData = ImageWrapper->GetCompressed(CurrentQuality);

			TBackChannelSharedPtr<FBackChannelOSCMessage> Msg = MakeShared<FBackChannelOSCMessage, ESPMode::ThreadSafe>(TEXT("/Screen"));
	
			Msg->Write(TEXT("Width"), Width);
			Msg->Write(TEXT("Height"), Height);
			Msg->Write(TEXT("Data"), JPGData);
			Msg->Write(TEXT("ImageCount"), ++NumSentImages);
			LocalConnection->SendPacket(Msg);

			UE_LOGF(LogRemoteSession, VeryVerbose, "Sent image %d in %.02f ms",
				NumSentImages, (FPlatformTime::Seconds() - TimeNow) * 1000.0);
		}
	}
}

FRemoteSessionImageChannel::FRemoteSessionImageChannel(ERemoteSessionChannelMode InRole, TSharedPtr<IBackChannelConnection, ESPMode::ThreadSafe> InConnection)
	: IRemoteSessionChannel(InRole, InConnection)
{
	Connection = InConnection;
	DecodedTextures[0] = nullptr;
	DecodedTextures[1] = nullptr;
	DecodedTextureIndex = MakeShared<TAtomic<int32>, ESPMode::ThreadSafe>(0);
	Role = InRole;
	LastDecodedImageIndex = 0;
	LastIncomingImageIndex = 0;

	BackgroundThread = nullptr;
	ScreenshotEvent = nullptr;
	ExitRequested = false;
	HaveConfiguredImageProvider = false;

	
	if (Role == ERemoteSessionChannelMode::Read)
	{
		auto Delegate = FBackChannelRouteDelegate::FDelegate::CreateRaw(this, &FRemoteSessionImageChannel::ReceiveHostImage);
		MessageCallbackHandle = InConnection->AddRouteDelegate(TEXT("/Screen"), Delegate);

		// #agrant todo: need equivalent
		// InConnection->SetMessageOptions(TEXT("/Screen"), 1);

		StartBackgroundThread();

		// Make sure this is something sensible
		InConnection->SetBufferSizes(0, 2 * 1024 * 1024);
	}
	else
	{
		// Big bugger for writing. #agrant-todo - need a way to auto set this?
		InConnection->SetBufferSizes(2 * 1024 * 1024, 0);
		ImageSender = MakeShared<FRemoteSessionImageChannel::FImageSender, ESPMode::ThreadSafe>(InConnection);
	}
}

FRemoteSessionImageChannel::~FRemoteSessionImageChannel()
{
	if (Role == ERemoteSessionChannelMode::Read)
	{
		TSharedPtr<IBackChannelConnection, ESPMode::ThreadSafe> LocalConnection = Connection.Pin();
		if (LocalConnection.IsValid())
		{
			// Remove the callback so it doesn't call back on an invalid this
			LocalConnection->RemoveRouteDelegate(TEXT("/Screen"), MessageCallbackHandle);
		}
		MessageCallbackHandle.Reset();

		ExitBackgroundThread();

		if (ScreenshotEvent)
		{
			//Cleanup the FEvent
			FGenericPlatformProcess::ReturnSynchEventToPool(ScreenshotEvent);
			ScreenshotEvent = nullptr;
		}

		if (BackgroundThread)
		{
			//Cleanup the worker thread
			delete BackgroundThread;
			BackgroundThread = nullptr;
		}
	}

	for (int32 i = 0; i < 2; i++)
	{
		if (DecodedTextures[i])
		{
			DecodedTextures[i]->RemoveFromRoot();
			DecodedTextures[i] = nullptr;
		}
	}
}

UTexture2D* FRemoteSessionImageChannel::GetHostScreen() const
{
	return DecodedTextures[DecodedTextureIndex->Load()];
}

void FRemoteSessionImageChannel::Tick(const float InDeltaTime)
{
	INC_DWORD_STAT(STAT_RSTickRate);

	if (Role == ERemoteSessionChannelMode::Write)
	{
		// If an image provider hasn't been configured yet then set a default
		if (!HaveConfiguredImageProvider && ImageProvider == nullptr)
		{
			SetFramebufferAsImageProvider();
		}

		if (ImageProvider)
		{
			ImageProvider->Tick(InDeltaTime);
		}
	}

	if (Role == ERemoteSessionChannelMode::Read)
	{
		SCOPE_CYCLE_COUNTER(STAT_RSTextureUpdate);

		TUniquePtr<FImageData> QueuedImage;

		{
			// Check to see if there are any queued images. We just care about the last
			FScopeLock ImageLock(&DecodedImageMutex);
			if (IncomingDecodedImages.Num())
			{				
				QueuedImage = MoveTemp(IncomingDecodedImages.Last());
				LastDecodedImageIndex = QueuedImage->ImageIndex;

				if (IncomingDecodedImages.Num() > 1)
				{
					UE_LOGF(LogRemoteSession, Verbose, "GT: Image %d is ready, discarding %d earlier images",
						QueuedImage->ImageIndex, IncomingDecodedImages.Num() - 1);
				}

				IncomingDecodedImages.Reset();
			}
		}

		// If an image was waiting...
		if (QueuedImage.IsValid())
		{
			int32 NextImage = DecodedTextureIndex->Load() == 0 ? 1 : 0;

			// create a texture if we don't have a suitable one
			if (DecodedTextures[NextImage] == nullptr || QueuedImage->Width != DecodedTextures[NextImage]->GetSizeX() || QueuedImage->Height != DecodedTextures[NextImage]->GetSizeY())
			{
				CreateTexture(NextImage, QueuedImage->Width, QueuedImage->Height);
			}

			// Update it on the render thread. There shouldn't (...) be any harm in GT code using it from this point
			FUpdateTextureRegion2D* Region = new FUpdateTextureRegion2D(0, 0, 0, 0, QueuedImage->Width, QueuedImage->Height);
			TArray<uint8>* TextureData = new TArray<uint8>(MoveTemp(QueuedImage->ImageData));
			TWeakPtr<TAtomic<int32>, ESPMode::ThreadSafe> DecodedTextureIndexWeaked = DecodedTextureIndex;

			// cleanup functions, gets executed on the render thread after UpdateTextureRegions
			TFunction<void(uint8*, const FUpdateTextureRegion2D*)> DataCleanupFunc = [DecodedTextureIndexWeaked, NextImage, TextureData](uint8* InTextureData, const FUpdateTextureRegion2D* InRegions)
			{
				if (TSharedPtr<TAtomic<int32>, ESPMode::ThreadSafe> DecodedTextureIndexPinned = DecodedTextureIndexWeaked.Pin())
				{
					DecodedTextureIndexPinned->Store(NextImage);
				}

				//this is executed later on the render thread, meanwhile TextureData might have changed
				delete TextureData;
				delete InRegions; 
			};

			DecodedTextures[NextImage]->UpdateTextureRegions(0, 1, Region, 4 * QueuedImage->Width, sizeof(FColor), TextureData->GetData(), DataCleanupFunc);
            
            const double ProcessTime = FPlatformTime::Seconds() - QueuedImage->TimeCreated;
            ReceiveStats.MaxImageProcessTime = FMath::Max(ReceiveStats.MaxImageProcessTime, ProcessTime);

			UE_LOGF(LogRemoteSession, VeryVerbose, "GT: Uploaded image %d",
				QueuedImage->ImageIndex);
		} //-V773

		const double TimeNow = FPlatformTime::Seconds();
		if (TimeNow - ReceiveStats.LastUpdateTime >= 1.0)
		{
			SET_DWORD_STAT(STAT_RSWaitingFrames, ReceiveStats.FramesWaiting);
			SET_DWORD_STAT(STAT_RSDiscardedFrames, ReceiveStats.FramesSkipped);
            SET_DWORD_STAT(STAT_RSMaxImageProcessTime, int32(ReceiveStats.MaxImageProcessTime * 1000));

			ReceiveStats = FRemoteSessionImageReceiveStats();
			ReceiveStats.LastUpdateTime = TimeNow;
		}
	}
}

void FRemoteSessionImageChannel::SetImageProvider(TSharedPtr<IRemoteSessionImageProvider> InImageProvider)
{
	if (Role == ERemoteSessionChannelMode::Write)
	{
		ImageProvider = InImageProvider;
	}

	HaveConfiguredImageProvider = true;
}

/** Sets up an image provider that mirrors the games framebuffer. Will be the default if no ImageProvider is set. */
void FRemoteSessionImageChannel::SetFramebufferAsImageProvider()
{
	TSharedPtr<FRemoteSessionFrameBufferImageProvider> NewImageProvider = MakeShared<FRemoteSessionFrameBufferImageProvider>(GetImageSender());

	{
		const URemoteSessionSettings* Settings = URemoteSessionSettings::StaticClass()->GetDefaultObject<URemoteSessionSettings>();

		NewImageProvider->SetCaptureFrameRate(Settings->FrameRate);
		SetCompressQuality(Settings->ImageQuality);
	}

	{
		TWeakPtr<SWindow> InputWindow;
		TWeakPtr<FSceneViewport> SceneViewport;
		FRemoteSessionUtils::FindSceneViewport(InputWindow, SceneViewport);

		if (TSharedPtr<FSceneViewport> SceneViewPortPinned = SceneViewport.Pin())
		{
			NewImageProvider->SetCaptureViewport(SceneViewPortPinned.ToSharedRef());
		}
	}

	SetImageProvider(NewImageProvider);
}



void FRemoteSessionImageChannel::SetCompressQuality(int32 InQuality)
{
	if (Role == ERemoteSessionChannelMode::Write)
	{
		ImageSender->SetCompressQuality(InQuality);
	}
}

void FRemoteSessionImageChannel::ReceiveHostImage(IBackChannelPacket& Message)
{
    SCOPE_CYCLE_COUNTER(STAT_RSReceiveTime);

    const double TimeNow = FPlatformTime::Seconds();

    static double LastReceipt = TimeNow;
    
    const double Delta = TimeNow - LastReceipt;
    LastReceipt = TimeNow;
    
	TUniquePtr<FImageData> ReceivedImage = MakeUnique<FImageData>();

	Message.Read(TEXT("Width"), ReceivedImage->Width);
	Message.Read(TEXT("Height"), ReceivedImage->Height);
	Message.Read(TEXT("Data"), ReceivedImage->ImageData);
	Message.Read(TEXT("ImageCount"), ReceivedImage->ImageIndex);
    ReceivedImage->TimeCreated = TimeNow;
    
    int32 Index = ReceivedImage->ImageIndex;

	if (ReceivedImage->Width == 0 ||
		ReceivedImage->Height == 0)
	{
		UE_LOGF(LogRemoteSession, Error, "FRemoteSessionImageChannel: Received zero width/height image. Ignoring!");
		return;
	}

	if (ReceivedImage->ImageData.Num() == 0)
	{
		UE_LOGF(LogRemoteSession, Error, "FRemoteSessionImageChannel: Received empty image data. Ignoring!");
		return;
	}

	{
		FScopeLock Lock(&IncomingImageMutex);
		IncomingEncodedImages.Add(MoveTemp(ReceivedImage));
	}
    

	if (ScreenshotEvent)
	{
        SCOPE_CYCLE_COUNTER(STAT_RSWakeupWait);
        
		// wake up the background thread.
		ScreenshotEvent->Trigger();
	}

	UE_LOGF(LogRemoteSession, VeryVerbose, "Received Next Image %d after %.03f secs. %d pending.",  Index, Delta, IncomingEncodedImages.Num());
}

void FRemoteSessionImageChannel::ProcessIncomingTextures()
{
	SCOPE_CYCLE_COUNTER(STAT_RSDecompressTime);

	TUniquePtr<FImageData> Image;
	const double StartTime = FPlatformTime::Seconds();
	{
		// check if there's anything to do, if not pause the background thread
		FScopeLock TaskLock(&IncomingImageMutex);

		if (IncomingEncodedImages.Num() == 0)
		{
			return;
		}

		ReceiveStats.FramesWaiting += IncomingEncodedImages.Num();
		ReceiveStats.FramesSkipped += IncomingEncodedImages.Num() - 1;

		// take the last image we don't care about the rest
		Image = MoveTemp(IncomingEncodedImages.Last());
		LastIncomingImageIndex = Image->ImageIndex;

		IncomingEncodedImages.Reset();

		UE_LOGF(LogRemoteSession, VeryVerbose, "Processing Image %d, discarding %d other pending images",
			Image->ImageIndex, IncomingEncodedImages.Num() - 1);
	}

	IImageWrapperModule* ImageWrapperModule = FModuleManager::GetModulePtr<IImageWrapperModule>(FName("ImageWrapper"));

	if (ImageWrapperModule != nullptr)
	{
		TSharedPtr<IImageWrapper> ImageWrapper = ImageWrapperModule->CreateImageWrapper(EImageFormat::JPEG);

		ImageWrapper->SetCompressed(Image->ImageData.GetData(), Image->ImageData.Num());

		TArray64<uint8> RawData;
		if (ImageWrapper->GetRaw(ERGBFormat::BGRA, 8, RawData))
		{
			TUniquePtr<FImageData> QueuedImage = MakeUnique<FImageData>();
			QueuedImage->Width = Image->Width;
			QueuedImage->Height = Image->Height;
			QueuedImage->ImageData = MoveTemp(RawData);
			QueuedImage->ImageIndex = Image->ImageIndex;
            QueuedImage->TimeCreated = Image->TimeCreated;

			{
				FScopeLock ImageLock(&DecodedImageMutex);
				if (LastDecodedImageIndex > 0 && IncomingDecodedImages.Num() > 1 && IncomingDecodedImages.Last()->ImageIndex > LastDecodedImageIndex)
				{
					IncomingDecodedImages.RemoveSingle(IncomingDecodedImages.Last());
				}
				IncomingDecodedImages.Add(MoveTemp(QueuedImage));

				UE_LOGF(LogRemoteSession, VeryVerbose, "finished decompressing image %d in %.02f ms (%d in queue)",
					Image->ImageIndex,
					(FPlatformTime::Seconds() - StartTime) * 1000.0,
					IncomingEncodedImages.Num());
			}
		}
	}
}

void FRemoteSessionImageChannel::CreateTexture(const int32 InSlot, const int32 InWidth, const int32 InHeight)
{
	if (DecodedTextures[InSlot])
	{
		DecodedTextures[InSlot]->RemoveFromRoot();
		DecodedTextures[InSlot] = nullptr;
	}

	DecodedTextures[InSlot] = UTexture2D::CreateTransient(InWidth, InHeight);

	DecodedTextures[InSlot]->AddToRoot();
	DecodedTextures[InSlot]->UpdateResource();

	UE_LOGF(LogRemoteSession, Log, "Created texture in slot %d %dx%d for incoming image", InSlot, InWidth, InHeight);
}

void FRemoteSessionImageChannel::StartBackgroundThread()
{
	check(BackgroundThread == nullptr);

	ExitRequested	= false;
	
	ScreenshotEvent = FGenericPlatformProcess::GetSynchEventFromPool(false);;

	BackgroundThread = FRunnableThread::Create(this, TEXT("RemoteSessionFrameBufferThread"), 1024 * 1024, TPri_AboveNormal);

}

void FRemoteSessionImageChannel::ExitBackgroundThread()
{
	ExitRequested = true;

	if (ScreenshotEvent)
	{
		ScreenshotEvent->Trigger();
	}

	if (BackgroundThread)
	{
		BackgroundThread->WaitForCompletion();
	}
}

uint32 FRemoteSessionImageChannel::Run()
{
	while(!ExitRequested)
	{
		// wait a maximum of 1 second or until triggered
		ScreenshotEvent->Wait(1000);

		ProcessIncomingTextures();
	}

	return 0;
}





