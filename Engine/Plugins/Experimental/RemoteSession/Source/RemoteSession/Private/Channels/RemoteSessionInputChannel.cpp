// Copyright Epic Games, Inc. All Rights Reserved.

#include "Channels/RemoteSessionInputChannel.h"
#include "BackChannel/IBackChannelConnection.h"
#include "Framework/Application/SlateApplication.h"
#include "BackChannel/IBackChannelPacket.h"
#include "MessageHandler/RecordingMessageHandler.h"
#include "Channels/RemoteSessionChannel.h"
#include "RemoteSessionUtils.h"
#include "MessageHandler/ProxyMessageHandler.h"


class FRemoteSessionInputChannelFactoryWorker : public IRemoteSessionChannelFactoryWorker
{
public:
	virtual TSharedPtr<IRemoteSessionChannel> Construct(ERemoteSessionChannelMode InMode, TSharedPtr<IBackChannelConnection, ESPMode::ThreadSafe> InConnection) const override
	{
		TSharedPtr<FRemoteSessionInputChannel> Channel = MakeShared<FRemoteSessionInputChannel>(InMode, InConnection);
		if (InMode == ERemoteSessionChannelMode::Read)
		{
			TWeakPtr<SWindow> InputWindow;
			TWeakPtr<FSceneViewport> SceneViewport;
			FRemoteSessionUtils::FindSceneViewport(InputWindow, SceneViewport);

			Channel->SetPlaybackWindow(InputWindow, SceneViewport);
		}

		return Channel;
	}
};

REGISTER_CHANNEL_FACTORY(FRemoteSessionInputChannel, FRemoteSessionInputChannelFactoryWorker, ERemoteSessionChannelMode::Read);

namespace RemoteSessionVars
{
	static FAutoConsoleVariable BlockLocalInput(TEXT("Remote.BlockLocalInput"), 0, TEXT("Don't accept local input when a host is connected"));
};

FRemoteSessionInputChannel::FRemoteSessionInputChannel(ERemoteSessionChannelMode InRole, TSharedPtr<IBackChannelConnection, ESPMode::ThreadSafe> InConnection)
	: IRemoteSessionChannel(InRole, InConnection)
{

	Connection = InConnection;
	Role = InRole;

	// if sending input replace the default message handler with a recording version, and set us as the
	// handler for that data 
	if (Role == ERemoteSessionChannelMode::Write)
	{
		// when we're recording (writing) we create a recording handler that receives platform-level input
		// and passes it through to the default UE handler
		DefaultHandler = FSlateApplication::Get().GetPlatformApplication()->GetMessageHandler();

		RecordingHandler = MakeShared<FRecordingMessageHandler>(DefaultHandler);
		RecordingHandler->SetRecordingHandler(this);

		FSlateApplication::Get().GetPlatformApplication()->SetMessageHandler(RecordingHandler.ToSharedRef());
	}
	else
	{
		TSharedPtr<FGenericApplicationMessageHandler> TargetHandler = FSlateApplication::Get().GetPlatformApplication()->GetMessageHandler();

		PlaybackHandler = MakeShared<FRecordingMessageHandler>(TargetHandler);
		
		auto Delegate = FBackChannelRouteDelegate::FDelegate::CreateRaw(this, &FRemoteSessionInputChannel::OnRemoteMessage);
		MessageCallbackHandle = Connection->AddRouteDelegate(TEXT("/MessageHandler/"), Delegate);

		//FSlateApplication::Get().GetPlatformApplication()->SetMessageHandler(PlaybackHandler.ToSharedRef());
	}
}

FRemoteSessionInputChannel::~FRemoteSessionInputChannel()
{
	if (Role == ERemoteSessionChannelMode::Read)
	{
		// Remove the callback so it doesn't call back on an invalid this
		Connection->RemoveRouteDelegate(TEXT("/MessageHandler/"), MessageCallbackHandle);
		MessageCallbackHandle.Reset();
	}

	// todo - is this ok? Might other things have changed the handler like we do?
	if (DefaultHandler.IsValid() && FSlateApplication::IsInitialized())
	{
		FSlateApplication::Get().GetPlatformApplication()->SetMessageHandler(DefaultHandler.ToSharedRef());
	}

	// should restore handler? What if something else changed it...
	if (RecordingHandler.IsValid())
	{
		RecordingHandler->SetRecordingHandler(nullptr);
	}
}


void FRemoteSessionInputChannel::SetPlaybackWindow(TWeakPtr<SWindow> InWindow, TWeakPtr<FSceneViewport> InViewport)
{
	if (PlaybackHandler.IsValid())
	{
		PlaybackHandler->SetPlaybackWindow(InWindow, InViewport);
	}
}

void FRemoteSessionInputChannel::TryRouteTouchMessageToWidget(bool bRouteMessageToWidget)
{
	if (PlaybackHandler.IsValid())
	{
		PlaybackHandler->TryRouteTouchMessageToWidget(bRouteMessageToWidget);
	}
}

FOnRouteTouchDownToWidgetFailedDelegate* FRemoteSessionInputChannel::GetOnRouteTouchDownToWidgetFailedDelegate()
{
	if (PlaybackHandler)
	{
		return &PlaybackHandler->GetOnRouteTouchDownToWidgetFailedDelegate();
	}

	// if the PlaybackHandler is null, this instance used for recording. The delegate is only useful for playback.
	return nullptr;
}

void FRemoteSessionInputChannel::SetInputRect(const FVector2D& TopLeft, const FVector2D& Extents)
{
	if (RecordingHandler.IsValid())
	{
		RecordingHandler->SetInputRect(TopLeft, Extents);
	}
}

void FRemoteSessionInputChannel::Tick(const float InDeltaTime)
{
	// everything else happens via messaging.
	if (Role == ERemoteSessionChannelMode::Read)
	{
		bool bBlockInput = RemoteSessionVars::BlockLocalInput->GetInt() > 0;

		if (bBlockInput && !PlaybackHandler->IsConsumingInput())
		{
			PlaybackHandler->SetConsumeInput(true);
		}
		else if (!bBlockInput && PlaybackHandler->IsConsumingInput())
		{
			PlaybackHandler->SetConsumeInput(false);
		}

		PlaybackHandler->Tick(InDeltaTime);
	}
	else
	{
		RecordingHandler->Tick(InDeltaTime);
	}
}

void FRemoteSessionInputChannel::RecordMessage(const TCHAR* MsgName, const TArray<uint8>& Data)
{
	if (Connection.IsValid())
	{
		// send as blobs
		FString Path = FString::Printf(TEXT("/MessageHandler/%s"), MsgName);

		TBackChannelSharedPtr<IBackChannelPacket> Packet = Connection->CreatePacket();
		Packet->SetPath(*Path);
		Packet->Write(TEXT("Data"), Data);

		Connection->SendPacket(Packet);
	}
}

void FRemoteSessionInputChannel::OnRemoteMessage(IBackChannelPacket& Message)
{
	FString MessageName = Message.GetPath();
	MessageName.RemoveFromStart(TEXT("/MessageHandler/"));

	TArray<uint8> MsgData;
	Message.Read(TEXT("Data"), MsgData);
	
	PlaybackHandler->PlayMessage(*MessageName, MoveTemp(MsgData));
}
