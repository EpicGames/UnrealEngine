// Copyright Epic Games, Inc. All Rights Reserved.

#include "Channels/RemoteSessionLiveLinkChannel.h"
#include "BackChannel/IBackChannelConnection.h"
#include "Channels/RemoteSessionChannel.h"
#include "RemoteSession.h"
#include "BackChannel/Protocol/OSC/BackChannelOSCMessage.h"


#include "Async/Async.h"

#if 0
#include "Channels/RemoteSessionChannel.h"
#include "Features/IModularFeatures.h"
#include "ILiveLinkClient.h"
#include "LiveLinkMessages.h"
#include "LiveLinkMessageBusSource.h"
#include "LiveLinkMessageBusSourceFactory.h"

#endif

class FRemoteSessionLiveLinkChannelFactoryWorker : public IRemoteSessionChannelFactoryWorker
{
public:
	TSharedPtr<IRemoteSessionChannel> Construct(ERemoteSessionChannelMode InMode, TBackChannelSharedPtr<IBackChannelConnection> InConnection) const
	{
		return MakeShared<FRemoteSessionLiveLinkChannel>(InMode, InConnection);
	}

};

REGISTER_CHANNEL_FACTORY(FRemoteSessionLiveLinkChannel, FRemoteSessionLiveLinkChannelFactoryWorker, ERemoteSessionChannelMode::Read);

#define LL_MESSAGE_ADDRESS TEXT("/RS.LiveLink")

FRemoteSessionLiveLinkChannel::FRemoteSessionLiveLinkChannel(ERemoteSessionChannelMode InRole, TSharedPtr<IBackChannelConnection, ESPMode::ThreadSafe> InConnection)
	: IRemoteSessionChannel(InRole, InConnection)
	, Connection(InConnection)
	, Role(InRole)
{
	auto Delegate = FBackChannelRouteDelegate::FDelegate::CreateRaw(this, &FRemoteSessionLiveLinkChannel::ReceiveLiveLinkHello);
	RouteHandle = InConnection->AddRouteDelegate(LL_MESSAGE_ADDRESS, Delegate);
}


FRemoteSessionLiveLinkChannel::~FRemoteSessionLiveLinkChannel()
{
	if (Role == ERemoteSessionChannelMode::Read)
	{
		// Remove the callback so it doesn't call back on an invalid this
		Connection->RemoveRouteDelegate(LL_MESSAGE_ADDRESS, RouteHandle);
	}
}

void FRemoteSessionLiveLinkChannel::Tick(const float InDeltaTime)
{
	// Nothing
}

void FRemoteSessionLiveLinkChannel::SendLiveLinkHello(const FStringView InSubjectName)
{
	if (Connection.IsValid())
    {
		TBackChannelSharedPtr<IBackChannelPacket> Msg = MakeShared<FBackChannelOSCMessage, ESPMode::ThreadSafe>(LL_MESSAGE_ADDRESS);
		Msg->Write(TEXT("SubjectName"), *FString(InSubjectName));
		Connection->SendPacket(Msg);

		UE_LOGF(LogRemoteSession, Log, "Sent LiveLinkHello for %ls", *FString(InSubjectName));
    }
}

void FRemoteSessionLiveLinkChannel::ReceiveLiveLinkHello(IBackChannelPacket& Message)
{
	FString SubjectName;
	Message.Read(TEXT("SubjectName"), SubjectName);

    AsyncTask(ENamedThreads::GameThread, [SubjectName]
	{
		UE_LOGF(LogRemoteSession, Log, "Received LiveLinkHello for %ls", *FString(SubjectName));

		#if 0
		IModularFeatures& ModularFeatures = IModularFeatures::Get();

		Connection->

		if (ModularFeatures.IsModularFeatureAvailable(ILiveLinkClient::ModularFeatureName))
		{
			ILiveLinkClient* LiveLinkClient = &ModularFeatures.GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName);
			TSharedPtr<FLiveLinkMessageBusSource> NewSource = MakeShared<FLiveLinkMessageBusSource>(FText::FromString(SubjectName), FText::FromString(Provider.MachineName), Provider.Address, 0);
			FGuid NewSourceGuid = LiveLinkClient->AddSource(NewSource);
			if (NewSourceGuid.IsValid())
			{
				if (ULiveLinkSourceSettings* Settings = LiveLinkClient->GetSourceSettings(NewSourceGuid))
				{
					Settings->ConnectionString = ULiveLinkMessageBusSourceFactory::CreateConnectionString(Provider);
					Settings->Factory = ULiveLinkMessageBusSourceFactory::StaticClass();
				}
			}
			SourceHandle.SetSourcePointer(NewSource);
		}
		#endif
	});
}

