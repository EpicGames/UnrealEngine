// Copyright Epic Games, Inc. All Rights Reserved.

#include "Channels/RemoteSessionXRTrackingChannel.h"
#include "ARSystem.h"
#include "RemoteSession.h"
#include "BackChannel/IBackChannelConnection.h"
#include "BackChannel/Protocol/OSC/BackChannelOSCMessage.h"
#include "ARBlueprintLibrary.h"
#include "Channels/RemoteSessionChannel.h"
#include "MessageHandler/Messages.h"

#include "Async/Async.h"
#include "Engine/Engine.h"
#include "RemoteSession.h"
#include "Serialization/MemoryReader.h"


class FRemoteSessionXRTrackingChannelFactoryWorker : public IRemoteSessionChannelFactoryWorker
{
public:
	TSharedPtr<IRemoteSessionChannel> Construct(ERemoteSessionChannelMode InMode, TBackChannelSharedPtr<IBackChannelConnection> InConnection) const
	{
		bool IsSupported = (InMode == ERemoteSessionChannelMode::Read) || UARBlueprintLibrary::IsSessionTypeSupported(EARSessionType::Orientation);
		if (IsSupported)
		{
			return MakeShared<FRemoteSessionXRTrackingChannel>(InMode, InConnection);
		}
		else
		{
			UE_LOGF(LogRemoteSession, Warning, "FRemoteSessionXRTrackingChannel does not support sending on this platform/device");
		}
		return TSharedPtr<IRemoteSessionChannel>();
	}

};

REGISTER_CHANNEL_FACTORY(FRemoteSessionXRTrackingChannel, FRemoteSessionXRTrackingChannelFactoryWorker, ERemoteSessionChannelMode::Read);

bool FXRTrackingProxy::EnumerateTrackedDevices(TArray<int32>& OutDevices, EXRTrackedDeviceType Type)
{
	if (Type == EXRTrackedDeviceType::Any || Type == EXRTrackedDeviceType::HeadMountedDisplay)
	{
		static const int32 DeviceId = IXRTrackingSystem::HMDDeviceId;
		OutDevices.Add(DeviceId);
		return true;
	}
	return false;
}

bool FXRTrackingProxy::GetCurrentPose(int32 DeviceId, FQuat& OutOrientation, FVector& OutPosition)
{
	OutOrientation = CachedTrackingToWorld.GetRotation();
	OutPosition = CachedTrackingToWorld.GetLocation();
	return true;
}

FName FXRTrackingProxy::GetSystemName() const
{
	static const FName RemoteSessionXRTrackingProxyName(TEXT("RemoteSessionXRTrackingProxy"));
	return RemoteSessionXRTrackingProxyName;
}

#define MESSAGE_ADDRESS TEXT("/XRTracking")

FRemoteSessionXRTrackingChannel::FRemoteSessionXRTrackingChannel(ERemoteSessionChannelMode InRole, TSharedPtr<IBackChannelConnection, ESPMode::ThreadSafe> InConnection)
	: IRemoteSessionChannel(InRole, InConnection)
	, Connection(InConnection)
	, Role(InRole)
	, ARSystemSupport(nullptr)
{
	Init();
}

FRemoteSessionXRTrackingChannel::FRemoteSessionXRTrackingChannel(ERemoteSessionChannelMode InRole, TSharedPtr<IBackChannelConnection, ESPMode::ThreadSafe> InConnection, IARSystemSupport* InARSystemSupport)
	: IRemoteSessionChannel(InRole, InConnection)
	, Connection(InConnection)
	, Role(InRole)
	, ARSystemSupport(InARSystemSupport)
{
	Init();
}

FRemoteSessionXRTrackingChannel::~FRemoteSessionXRTrackingChannel()
{
	if (Role == ERemoteSessionChannelMode::Read)
	{
		// Remove the callback so it doesn't call back on an invalid this
		Connection->RemoveRouteDelegate(MESSAGE_ADDRESS, MessageCallbackHandle);

        if (GEngine != nullptr)
        {
            // Reset the engine back to what it was before
            GEngine->XRSystem = XRSystem;
        }
	}
	// Release our xr trackers
	XRSystem = nullptr;
	ProxyXRSystem = nullptr;
}

void FRemoteSessionXRTrackingChannel::Init()
{
	// If we are sending, we grab the data from GEngine->XRSystem, otherwise we back the current one up for restore later
	XRSystem = GEngine->XRSystem;

	if (Role == ERemoteSessionChannelMode::Read)
	{
		// Make the proxy and set GEngine->XRSystem to it
		ProxyXRSystem = MakeShared<FXRTrackingProxy, ESPMode::ThreadSafe>(ARSystemSupport);
		GEngine->XRSystem = ProxyXRSystem;

		auto Delegate = FBackChannelRouteDelegate::FDelegate::CreateRaw(this, &FRemoteSessionXRTrackingChannel::ReceiveXRTracking);
		MessageCallbackHandle = Connection->AddRouteDelegate(MESSAGE_ADDRESS, Delegate);

		// #agrant todo: need equivalent
		// Connection->SetMessageOptions(MESSAGE_ADDRESS, 1);
	}
	else
	{
#if PLATFORM_IOS
		if (UARBlueprintLibrary::GetARSessionStatus().Status != EARSessionStatus::Running)
		{
			UARSessionConfig* Config = NewObject<UARSessionConfig>();
			UARBlueprintLibrary::StartARSession(Config);
		}
#endif
	}
}

void FRemoteSessionXRTrackingChannel::Tick(const float InDeltaTime)
{
	// Inbound data gets handled as callbacks
	if (Role == ERemoteSessionChannelMode::Write)
	{
		SendXRTracking();
	}
}

void FRemoteSessionXRTrackingChannel::SendXRTracking()
{
	if (Connection.IsValid())
    {
        if (XRSystem.IsValid() && XRSystem->IsTracking(IXRTrackingSystem::HMDDeviceId))
        {
            FVector Location;
            FQuat Orientation;
            if (XRSystem->GetCurrentPose(IXRTrackingSystem::HMDDeviceId, Orientation, Location))
            {
                FRotator Rotation(Orientation);

                TwoParamMsg<FVector, FRotator> MsgParam(Location, Rotation);
				TBackChannelSharedPtr<IBackChannelPacket> Msg = MakeShared<FBackChannelOSCMessage, ESPMode::ThreadSafe>(MESSAGE_ADDRESS);
                Msg->Write(TEXT("XRData"), MsgParam.AsData());

                Connection->SendPacket(Msg);
                
                UE_LOGF(LogRemoteSession, VeryVerbose, "Sent Rotation (%.02f,%.02f,%.02f)", Rotation.Pitch,Rotation.Yaw,Rotation.Roll);
            }
            else
            {
                 UE_LOGF(LogRemoteSession, Warning, "Failed to get XRPose");
            }
        }
        else
        {
            UE_LOGF(LogRemoteSession, Warning, "XR Tracking not available to send");
        }
    }
}

void FRemoteSessionXRTrackingChannel::ReceiveXRTracking(IBackChannelPacket& Message)
{
	if (!ProxyXRSystem.IsValid())
	{
        UE_LOGF(LogRemoteSession, Warning, "XRProxy is invalid. Cannot receive pose");
		return;
	}

	TUniquePtr<TArray<uint8>> DataCopy = MakeUnique<TArray<uint8>>();
    TWeakPtr<IXRTrackingSystem, ESPMode::ThreadSafe> TaskXRSystem = ProxyXRSystem;

	Message.Read(TEXT("XRData"), *DataCopy);

    AsyncTask(ENamedThreads::GameThread, [TaskXRSystem, DataCopy=MoveTemp(DataCopy)]
	{
		TSharedPtr<IXRTrackingSystem, ESPMode::ThreadSafe> TaskXRSystemPinned = TaskXRSystem.Pin();
		if (TaskXRSystemPinned)
		{
			FMemoryReader Ar(*DataCopy);
			TwoParamMsg<FVector, FRotator> MsgParam(Ar);

			UE_LOGF(LogRemoteSession, VeryVerbose, "Received Rotation (%.02f,%.02f,%.02f)", MsgParam.Param2.Pitch, MsgParam.Param2.Yaw, MsgParam.Param2.Roll);

			FTransform NewTransform(MsgParam.Param2, MsgParam.Param1);
			TaskXRSystemPinned->UpdateTrackingToWorldTransform(NewTransform);
		}
	});
}

