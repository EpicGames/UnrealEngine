// Copyright Epic Games, Inc. All Rights Reserved.

#include "ImageProviders/RemoteSessionFrameBufferImageProvider.h"
#include "FrameGrabber.h"
#include "Async/Async.h"
#include "Slate/SceneViewport.h"
#include "Widgets/SWindow.h"


static int32 FramerateSetting = 0;
static FAutoConsoleVariableRef CVarFramerateOverride(
	TEXT("remote.framerate"), FramerateSetting,
	TEXT("Sets framerate"),
	ECVF_Default);

static int32 FrameGrabberResX = 0;
static FAutoConsoleVariableRef CVarResXOverride(
	TEXT("remote.framegrabber.resx"), FrameGrabberResX,
	TEXT("Sets the desired X resolution"),
	ECVF_Default);

static int32 FrameGrabberResY = 0;
static FAutoConsoleVariableRef CVarResYOverride(
	TEXT("remote.framegrabber.resy"), FrameGrabberResY,
	TEXT("Sets the desired Y resolution"),
	ECVF_Default);

FRemoteSessionFrameBufferImageProvider::FRemoteSessionFrameBufferImageProvider(TSharedPtr<FRemoteSessionImageChannel::FImageSender, ESPMode::ThreadSafe> InImageSender)
{
	ImageSender = InImageSender;
	LastSentImageTime = 0.0;
	ViewportResized = false;
	NumDecodingTasks = MakeShared<FThreadSafeCounter, ESPMode::ThreadSafe>();
}

FRemoteSessionFrameBufferImageProvider::~FRemoteSessionFrameBufferImageProvider()
{
	if (TSharedPtr<FSceneViewport> PreviousSceneViewportPinned = SceneViewport.Pin())
	{
		PreviousSceneViewportPinned->SetOnSceneViewportResizeDel(FOnSceneViewportResize());
	}
	if (TSharedPtr<SWindow> SceneViewportWindowPin = SceneViewportWindow.Pin())
	{
		SceneViewportWindowPin->GetOnWindowClosedEvent().RemoveAll(this);
	}
	ReleaseFrameGrabber();
}

void FRemoteSessionFrameBufferImageProvider::ReleaseFrameGrabber()
{
	if (FrameGrabber.IsValid())
	{
		FrameGrabber->Shutdown();
		FrameGrabber = nullptr;
	}
}

void FRemoteSessionFrameBufferImageProvider::SetCaptureFrameRate(int32 InFramerate)
{
	// Set our framerate and quality cvars, if the user hasn't modified them
	if (FramerateSetting == 0)
	{
		CVarFramerateOverride->Set(InFramerate);
	}
}

void FRemoteSessionFrameBufferImageProvider::SetCaptureViewport(TSharedRef<FSceneViewport> Viewport)
{
	if (Viewport != SceneViewport)
	{
		if (TSharedPtr<FSceneViewport> PreviousSceneViewportPinned = SceneViewport.Pin())
		{
			PreviousSceneViewportPinned->SetOnSceneViewportResizeDel(FOnSceneViewportResize());
		}
		if (TSharedPtr<SWindow> SceneViewportWindowPin = SceneViewportWindow.Pin())
		{
			SceneViewportWindowPin->GetOnWindowClosedEvent().RemoveAll(this);
		}

		SceneViewportWindow.Reset();
		SceneViewport = Viewport;

		if (TSharedPtr<SWindow> Window = Viewport->FindWindow())
		{
			SceneViewportWindow = Window;
			Window->GetOnWindowClosedEvent().AddRaw(this, &FRemoteSessionFrameBufferImageProvider::OnWindowClosedEvent);
		}

		CreateFrameGrabber(Viewport);

		// set the listener for the window resize event
		Viewport->SetOnSceneViewportResizeDel(FOnSceneViewportResize::CreateRaw(this, &FRemoteSessionFrameBufferImageProvider::OnViewportResized));
	}
}

void FRemoteSessionFrameBufferImageProvider::OnWindowClosedEvent(const TSharedRef<SWindow>&)
{
	ReleaseFrameGrabber();
	SceneViewport.Reset();
}

void FRemoteSessionFrameBufferImageProvider::CreateFrameGrabber(TSharedRef<FSceneViewport> Viewport)
{
	ReleaseFrameGrabber();

	// For times when we want a specific resolution
	FIntPoint FrameGrabberSize = Viewport->GetSize();
	if (FrameGrabberResX > 0)
	{
		FrameGrabberSize.X = FrameGrabberResX;
	}
	if (FrameGrabberResY > 0)
	{
		FrameGrabberSize.Y = FrameGrabberResY;
	}

	FrameGrabber = MakeShared<FFrameGrabber>(Viewport, FrameGrabberSize);
	FrameGrabber->StartCapturingFrames();
}

void FRemoteSessionFrameBufferImageProvider::Tick(const float InDeltaTime)
{
	const int kMaxPendingFrames = 1;

	const double TimeNow = FPlatformTime::Seconds();
	const double ElapsedImageTimeMS = (TimeNow - LastSentImageTime) * 1000;
	const int32 DesiredFrameTimeMS = 1000 / FramerateSetting;

	if (ElapsedImageTimeMS < DesiredFrameTimeMS)
	{
		return;
	}

	if (TimeNow - CaptureStats.LastUpdateTime >= 1.0)
	{	
		SET_DWORD_STAT(STAT_RSCaptureCount, CaptureStats.FramesCaptured);
		SET_DWORD_STAT(STAT_RSSkippedFrames, CaptureStats.FramesSkipped);

		CaptureStats = FRemoteSesstionImageCaptureStats();
		CaptureStats.LastUpdateTime = TimeNow;
	}	

	SCOPE_CYCLE_COUNTER(STAT_RSCaptureTime);

	if (FrameGrabber.IsValid())
	{
		if (ViewportResized)
		{
			ReleaseFrameGrabber();
			if (TSharedPtr<FSceneViewport> SceneViewportPinned = SceneViewport.Pin())
			{
				CreateFrameGrabber(SceneViewportPinned.ToSharedRef());
			}
			ViewportResized = false;
		}		

		if (FrameGrabber)
		{
			FrameGrabber->CaptureThisFrame(FFramePayloadPtr());

			TArray<FCapturedFrameData> Frames = FrameGrabber->GetCapturedFrames();

			if (Frames.Num())
			{		
				// Encoding/decoding can take longer than a frame, so skip if we're still processing the previous frame
				if (NumDecodingTasks->GetValue() <= kMaxPendingFrames)
				{
					NumDecodingTasks->Increment();

					CaptureStats.FramesCaptured++;

					FCapturedFrameData& LastFrame = Frames.Last();
					FIntPoint Size = LastFrame.BufferSize;
					TArray<FColor> ColorData = MoveTemp(LastFrame.ColorBuffer);
					TWeakPtr<FThreadSafeCounter, ESPMode::ThreadSafe> WeakNumDecodingTasks = NumDecodingTasks;
					TWeakPtr<FRemoteSessionImageChannel::FImageSender, ESPMode::ThreadSafe> WeakImageSender = ImageSender;

					AsyncTask(ENamedThreads::AnyHiPriThreadHiPriTask, [WeakNumDecodingTasks, WeakImageSender, Size, ColorData=MoveTemp(ColorData)]() mutable
					{
						SCOPE_CYCLE_COUNTER(STAT_RSCompressTime);

						if (WeakImageSender.IsValid())
						{
							if (TSharedPtr<FThreadSafeCounter, ESPMode::ThreadSafe> NumDecodingTasksPinned = WeakNumDecodingTasks.Pin())
							{
								for (FColor& Color : ColorData)
								{
									Color.A = 255;
								}

								if (TSharedPtr<FRemoteSessionImageChannel::FImageSender, ESPMode::ThreadSafe> ImageSenderPinned = WeakImageSender.Pin())
								{
									ImageSenderPinned->SendRawImageToClients(Size.X, Size.Y, ColorData.GetData(), ColorData.GetAllocatedSize());
								}

								NumDecodingTasksPinned->Decrement();
							}
						}

					});
				}
				else
				{
					CaptureStats.FramesSkipped++;
				}

				LastSentImageTime = FPlatformTime::Seconds();
			}
			else
			{
				CaptureStats.FramesSkipped++;
			}
		}
	}
}

void FRemoteSessionFrameBufferImageProvider::OnViewportResized(FVector2D NewSize)
{
	ViewportResized = true;
}
