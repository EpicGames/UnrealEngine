// Copyright Epic Games, Inc. All Rights Reserved.

#include "ImageProviders/RemoteSessionMediaOutput.h"

#include "Channels/RemoteSessionImageChannel.h"
#include "IRemoteSessionRole.h"
#include "Modules/ModuleManager.h"
#include "RemoteSessionModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RemoteSessionMediaOutput)


UMediaCapture* URemoteSessionMediaOutput::CreateMediaCaptureImpl()
{
	if (!ImageChannel.IsValid())
	{
		FRemoteSessionModule& RemoteSession = FModuleManager::GetModuleChecked<FRemoteSessionModule>("RemoteSession");
		if (TSharedPtr<IRemoteSessionRole> Host = RemoteSession.GetHost())
		{
			if (TSharedPtr<FRemoteSessionImageChannel> FBChannel = Host->GetChannel<FRemoteSessionImageChannel>())
			{
				ImageChannel = FBChannel;
			}
		}
	}

	URemoteSessionMediaCapture* Result = nullptr;
	if (ImageChannel.IsValid())
	{
		Result = NewObject<URemoteSessionMediaCapture>();
		Result->SetMediaOutput(this);
	}
	return Result;
}

void URemoteSessionMediaOutput::SetImageChannel(TWeakPtr<FRemoteSessionImageChannel> InImageChannel)
{
	ImageChannel = InImageChannel;
}

void URemoteSessionMediaCapture::OnFrameCaptured_RenderingThread(const FCaptureBaseData& InBaseData, TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> InUserData, void* InBuffer, int32 Width, int32 Height, int32 BytesPerRow)
{
	if (ImageChannel)
	{
		ImageChannel->GetImageSender()->SendRawImageToClients(Width, Height, InBuffer, BytesPerRow * Height, BytesPerRow);
	}
}

bool URemoteSessionMediaCapture::InitializeCapture()
{
	CacheValues();

	SetState(EMediaCaptureState::Capturing);
	return true;
}

void URemoteSessionMediaCapture::CacheValues()
{
	URemoteSessionMediaOutput* RemoteSessionMediaOutput = CastChecked<URemoteSessionMediaOutput>(MediaOutput);
	ImageChannel = RemoteSessionMediaOutput->GetImageChannel().Pin();
}
