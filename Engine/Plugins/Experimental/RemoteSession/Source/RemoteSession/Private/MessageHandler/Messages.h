// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Serialization/BufferArchive.h"

struct NoParamMsg
{
	NoParamMsg()
	{
	}

	NoParamMsg(FArchive& Ar)
	{
	}

	TArray<uint8> AsData()
	{
		FBufferArchive MemAr;
		return MoveTemp(MemAr);
	}
};

template <typename P1, typename P2>
struct TwoParamMsg
{
	P1	Param1;
	P2	Param2;

	TwoParamMsg(FArchive& Ar)
	{
		Param1 = P1();
		Param2 = P2();
		Ar << Param1;
		Ar << Param2;
	}

	TwoParamMsg(P1 InParam1, P2 InParam2)
	{
		Param1 = InParam1;
		Param2 = InParam2;
	}

	TArray<uint8> AsData()
	{
		FBufferArchive MemAr;
		MemAr << Param1 << Param2;
		return MoveTemp(MemAr);
	}
};

template <typename P1, typename P2, typename P3>
struct ThreeParamMsg
{
	P1	Param1;
	P2	Param2;
	P3	Param3;

	ThreeParamMsg(FArchive& Ar)
	{
		Param1 = P1();
		Param2 = P2();
		Param3 = P3();
		Ar << Param1;
		Ar << Param2;
		Ar << Param3;
	}

	ThreeParamMsg(P1 InParam1, P2 InParam2, P3 InParam3)
	{
		Param1 = InParam1;
		Param2 = InParam2;
		Param3 = InParam3;
	}

	TArray<uint8> AsData()
	{
		FBufferArchive MemAr;
		MemAr << Param1 << Param2 << Param3;
		return MoveTemp(MemAr);
	}
};

template <typename P1, typename P2, typename P3, typename P4>
struct FourParamMsg
{
	P1	Param1;
	P2	Param2;
	P3	Param3;
	P4	Param4;

	FourParamMsg(FArchive& Ar)
	{
		Param1 = P1();
		Param2 = P2();
		Param3 = P3();
		Param4 = P4();
		Ar << Param1;
		Ar << Param2;
		Ar << Param3;
		Ar << Param4;
	}

	FourParamMsg(P1 InParam1, P2 InParam2, P3 InParam3, P4 InParam4)
	{
		Param1 = InParam1;
		Param2 = InParam2;
		Param3 = InParam3;
		Param4 = InParam4;
	}

	TArray<uint8> AsData()
	{
		FBufferArchive MemAr;
		MemAr << Param1 << Param2 << Param3 << Param4;
		return MoveTemp(MemAr);
	}
};

template <typename P1, typename P2, typename P3, typename P4, typename P5>
struct FiveParamMsg
{
	P1	Param1;
	P2	Param2;
	P3	Param3;
	P4	Param4;
	P5	Param5;

	FiveParamMsg(FArchive& Ar)
	{
		Param1 = P1();
		Param2 = P2();
		Param3 = P3();
		Param4 = P4();
		Param5 = P5();
		Ar << Param1;
		Ar << Param2;
		Ar << Param3;
		Ar << Param4;
		Ar << Param5;
	}

	FiveParamMsg(P1 InParam1, P2 InParam2, P3 InParam3, P4 InParam4, P5 InParam5)
	{
		Param1 = InParam1;
		Param2 = InParam2;
		Param3 = InParam3;
		Param4 = InParam4;
		Param5 = InParam5;
	}

	TArray<uint8> AsData()
	{
		FBufferArchive MemAr;
		MemAr << Param1 << Param2 << Param3 << Param4 << Param5;
		return MoveTemp(MemAr);
	}
};
