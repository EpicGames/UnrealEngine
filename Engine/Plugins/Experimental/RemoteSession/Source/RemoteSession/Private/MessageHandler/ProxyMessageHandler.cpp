// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProxyMessageHandler.h"

FProxyMessageHandler::FProxyMessageHandler(const TSharedPtr<FGenericApplicationMessageHandler>& InTargetHandler)
	: TargetHandler(InTargetHandler)
{

}

FProxyMessageHandler::~FProxyMessageHandler()
{
}

void FProxyMessageHandler::SetTargetHandler(const TSharedPtr<FGenericApplicationMessageHandler>& InTargetHandler)
{
	TargetHandler = InTargetHandler;
}

const TSharedPtr<FGenericApplicationMessageHandler> FProxyMessageHandler::GetTargetHandler() const
{
	return TargetHandler;
}


bool FProxyMessageHandler::ShouldProcessUserInputMessages(const TSharedPtr< FGenericWindow >& PlatformWindow) const
{
	if (TargetHandler.IsValid())
	{
		return TargetHandler->ShouldProcessUserInputMessages(PlatformWindow);
	}

	return false;
}

bool FProxyMessageHandler::OnKeyChar(const TCHAR Character, const bool IsRepeat)
{
	if (TargetHandler.IsValid())
	{
		return TargetHandler->OnKeyChar(Character, IsRepeat);
	}

	return false;
}

bool FProxyMessageHandler::OnKeyDown(const int32 KeyCode, const uint32 CharacterCode, const bool IsRepeat)
{
	if (TargetHandler.IsValid())
	{
		return TargetHandler->OnKeyDown(KeyCode, CharacterCode, IsRepeat);
	}

	return false;
}

bool FProxyMessageHandler::OnKeyUp(const int32 KeyCode, const uint32 CharacterCode, const bool IsRepeat)
{
	if (TargetHandler.IsValid())
	{
		return TargetHandler->OnKeyUp(KeyCode, CharacterCode, IsRepeat);
	}

	return false;
}

bool FProxyMessageHandler::OnMouseDown(const TSharedPtr< FGenericWindow >& Window, const EMouseButtons::Type Button)
{
	if (TargetHandler.IsValid())
	{
		return TargetHandler->OnMouseDown(Window, Button);
	}

	return false;
}

bool FProxyMessageHandler::OnMouseDown(const TSharedPtr< FGenericWindow >& Window, const EMouseButtons::Type Button, const FVector2D CursorPos)
{
	if (TargetHandler.IsValid())
	{
		return TargetHandler->OnMouseDown(Window, Button, CursorPos);
	}

	return false;
}

bool FProxyMessageHandler::OnMouseUp(const EMouseButtons::Type Button)
{
	if (TargetHandler.IsValid())
	{
		return TargetHandler->OnMouseUp(Button);
	}

	return false;
}

bool FProxyMessageHandler::OnMouseUp(const EMouseButtons::Type Button, const FVector2D CursorPos)
{
	if (TargetHandler.IsValid())
	{
		return TargetHandler->OnMouseUp(Button, CursorPos);
	}

	return false;
}

bool FProxyMessageHandler::OnMouseDoubleClick(const TSharedPtr< FGenericWindow >& Window, const EMouseButtons::Type Button)
{
	if (TargetHandler.IsValid())
	{
		return TargetHandler->OnMouseDoubleClick(Window, Button);
	}

	return false;
}

bool FProxyMessageHandler::OnMouseDoubleClick(const TSharedPtr< FGenericWindow >& Window, const EMouseButtons::Type Button, const FVector2D CursorPos)
{
	if (TargetHandler.IsValid())
	{
		return TargetHandler->OnMouseDoubleClick(Window, Button, CursorPos);
	}

	return false;
}

bool FProxyMessageHandler::OnMouseWheel(const float Delta)
{
	if (TargetHandler.IsValid())
	{
		return TargetHandler->OnMouseWheel(Delta);
	}

	return false;
}

bool FProxyMessageHandler::OnMouseWheel(const float Delta, const FVector2D CursorPos)
{
	if (TargetHandler.IsValid())
	{
		return TargetHandler->OnMouseWheel(Delta, CursorPos);
	}

	return false;
}

bool FProxyMessageHandler::OnMouseMove()
{
	if (TargetHandler.IsValid())
	{
		return TargetHandler->OnMouseMove();
	}

	return false;
}

bool FProxyMessageHandler::OnRawMouseMove(const int32 X, const int32 Y)
{
	if (TargetHandler.IsValid())
	{
		return TargetHandler->OnRawMouseMove(X,Y);
	}

	return false;
}

bool FProxyMessageHandler::OnCursorSet()
{
	if (TargetHandler.IsValid())
	{
		return TargetHandler->OnCursorSet();
	}

	return false;
}

bool FProxyMessageHandler::OnControllerAnalog(FGamepadKeyNames::Type KeyName, FPlatformUserId PlatformUserId, FInputDeviceId InputDeviceId, float AnalogValue)
{
	if (TargetHandler.IsValid())
	{
		return TargetHandler->OnControllerAnalog(KeyName, PlatformUserId, InputDeviceId, AnalogValue);
	}

	return false;
}

bool FProxyMessageHandler::OnControllerButtonPressed(FGamepadKeyNames::Type KeyName, FPlatformUserId PlatformUserId, FInputDeviceId InputDeviceId, bool IsRepeat)
{
	if (TargetHandler.IsValid())
	{
		return TargetHandler->OnControllerButtonPressed(KeyName, PlatformUserId, InputDeviceId, IsRepeat);
	}

	return false;
}

bool FProxyMessageHandler::OnControllerButtonReleased(FGamepadKeyNames::Type KeyName, FPlatformUserId PlatformUserId, FInputDeviceId InputDeviceId, bool IsRepeat)
{
	if (TargetHandler.IsValid())
	{
		return TargetHandler->OnControllerButtonReleased(KeyName, PlatformUserId, InputDeviceId, IsRepeat);
	}

	return false;
}


void FProxyMessageHandler::OnBeginGesture()
{
	if (TargetHandler.IsValid())
	{
		TargetHandler->OnBeginGesture();
	}
}

bool FProxyMessageHandler::OnTouchGesture(EGestureEvent GestureType, const FVector2D& Delta, float WheelDelta, bool bIsDirectionInvertedFromDevice)
{
	return false;
}

void FProxyMessageHandler::OnEndGesture()
{
	if (TargetHandler.IsValid())
	{
		TargetHandler->OnEndGesture();
	}
}

bool FProxyMessageHandler::OnTouchStarted(const TSharedPtr< FGenericWindow >& Window, const FVector2D& Location, float Force, int32 TouchIndex, int32 ControllerId)
{
	if (TargetHandler.IsValid())
	{
		return TargetHandler->OnTouchStarted(Window, Location, Force, TouchIndex, ControllerId);
	}

	return false;
}

bool FProxyMessageHandler::OnTouchMoved(const FVector2D& Location, float Force, int32 TouchIndex, int32 ControllerId)
{
	if (TargetHandler.IsValid())
	{
		return TargetHandler->OnTouchMoved(Location, Force, TouchIndex, ControllerId);
	}

	return false;
}

bool FProxyMessageHandler::OnTouchEnded(const FVector2D& Location, int32 TouchIndex, int32 ControllerId)
{
	if (TargetHandler.IsValid())
	{
		return TargetHandler->OnTouchEnded(Location, TouchIndex, ControllerId);
	}

	return false;
}

bool FProxyMessageHandler::OnTouchForceChanged(const FVector2D& Location, float Force, int32 TouchIndex, int32 ControllerId)
{
	if (TargetHandler.IsValid())
	{
		return TargetHandler->OnTouchForceChanged(Location, Force, TouchIndex, ControllerId);
	}

	return false;
}

bool FProxyMessageHandler::OnTouchFirstMove(const FVector2D& Location, float Force, int32 TouchIndex, int32 ControllerId)
{
	if (TargetHandler.IsValid())
	{
		return TargetHandler->OnTouchFirstMove(Location, Force, TouchIndex, ControllerId);
	}

	return false;
}

void FProxyMessageHandler::ShouldSimulateGesture(EGestureEvent Gesture, bool bEnable)
{
	if (TargetHandler.IsValid())
	{
		TargetHandler->ShouldSimulateGesture(Gesture, bEnable);
	}
}

bool FProxyMessageHandler::OnMotionDetected(const FVector& Tilt, const FVector& RotationRate, const FVector& Gravity, const FVector& Acceleration, int32 ControllerId)
{
	if (TargetHandler.IsValid())
	{
		return TargetHandler->OnMotionDetected(Tilt, RotationRate, Gravity, Acceleration, ControllerId);
	}

	return false;
}

bool FProxyMessageHandler::OnSizeChanged(const TSharedRef< FGenericWindow >& Window, const int32 Width, const int32 Height, bool bWasMinimized /*= false*/)
{
	if (TargetHandler.IsValid())
	{
		return TargetHandler->OnSizeChanged(Window, Width, Height, bWasMinimized);
	}

	return false;
}

void FProxyMessageHandler::OnOSPaint(const TSharedRef<FGenericWindow>& Window)
{
	if (TargetHandler.IsValid())
	{
		TargetHandler->OnOSPaint(Window);
	}
}

FWindowSizeLimits FProxyMessageHandler::GetSizeLimitsForWindow(const TSharedRef<FGenericWindow>& Window) const
{
	if (TargetHandler.IsValid())
	{
		return TargetHandler->GetSizeLimitsForWindow(Window);
	}

	return FWindowSizeLimits();
}

void FProxyMessageHandler::OnResizingWindow(const TSharedRef< FGenericWindow >& Window)
{
	if (TargetHandler.IsValid())
	{
		TargetHandler->OnResizingWindow(Window);
	}
}

bool FProxyMessageHandler::BeginReshapingWindow(const TSharedRef< FGenericWindow >& Window)
{
	if (TargetHandler.IsValid())
	{
		return TargetHandler->BeginReshapingWindow(Window);
	}

	return true;
}

void FProxyMessageHandler::FinishedReshapingWindow(const TSharedRef< FGenericWindow >& Window)
{
	if (TargetHandler.IsValid())
	{
		TargetHandler->FinishedReshapingWindow(Window);
	}
}

void FProxyMessageHandler::HandleDPIScaleChanged(const TSharedRef< FGenericWindow >& Window)
{
	if (TargetHandler.IsValid())
	{
		return TargetHandler->HandleDPIScaleChanged(Window);
	}
}

void FProxyMessageHandler::OnMovedWindow(const TSharedRef< FGenericWindow >& Window, const int32 X, const int32 Y)
{
	if (TargetHandler.IsValid())
	{
		return TargetHandler->OnMovedWindow(Window, X, Y);
	}
}

bool FProxyMessageHandler::OnWindowActivationChanged(const TSharedRef< FGenericWindow >& Window, const EWindowActivation ActivationType)
{
	if (TargetHandler.IsValid())
	{
		return TargetHandler->OnWindowActivationChanged(Window, ActivationType);
	}

	return false;
}

bool FProxyMessageHandler::OnApplicationActivationChanged(const bool IsActive)
{
	if (TargetHandler.IsValid())
	{
		return TargetHandler->OnApplicationActivationChanged(IsActive);
	}

	return false;
}

bool FProxyMessageHandler::OnConvertibleLaptopModeChanged()
{
	if (TargetHandler.IsValid())
	{
		return TargetHandler->OnConvertibleLaptopModeChanged();
	}

	return false;
}

EWindowZone::Type FProxyMessageHandler::GetWindowZoneForPoint(const TSharedRef< FGenericWindow >& Window, const int32 X, const int32 Y)
{
	if (TargetHandler.IsValid())
	{
		return TargetHandler->GetWindowZoneForPoint(Window, X, Y);
	}

	return EWindowZone::NotInWindow;
}

void FProxyMessageHandler::OnWindowClose(const TSharedRef< FGenericWindow >& Window)
{
	if (TargetHandler.IsValid())
	{
		TargetHandler->OnWindowClose(Window);
	}
}

EDropEffect::Type FProxyMessageHandler::OnDragEnterText(const TSharedRef< FGenericWindow >& Window, const FString& Text)
{
	if (TargetHandler.IsValid())
	{
		return TargetHandler->OnDragEnterText(Window, Text);
	}

	return EDropEffect::None;
}

EDropEffect::Type FProxyMessageHandler::OnDragEnterFiles(const TSharedRef< FGenericWindow >& Window, const TArray< FString >& Files)
{
	if (TargetHandler.IsValid())
	{
		return TargetHandler->OnDragEnterFiles(Window, Files);
	}

	return EDropEffect::None;
}

EDropEffect::Type FProxyMessageHandler::OnDragEnterExternal(const TSharedRef< FGenericWindow >& Window, const FString& Text, const TArray< FString >& Files)
{
	if (TargetHandler.IsValid())
	{
		return TargetHandler->OnDragEnterExternal(Window, Text, Files);
	}

	return EDropEffect::None;
}

EDropEffect::Type FProxyMessageHandler::OnDragOver(const TSharedPtr< FGenericWindow >& Window)
{
	if (TargetHandler.IsValid())
	{
		return TargetHandler->OnDragOver(Window);
	}

	return EDropEffect::None;
}

void FProxyMessageHandler::OnDragLeave(const TSharedPtr< FGenericWindow >& Window)
{
	if (TargetHandler.IsValid())
	{
		TargetHandler->OnDragLeave(Window);
	}
}

EDropEffect::Type FProxyMessageHandler::OnDragDrop(const TSharedPtr< FGenericWindow >& Window)
{
	if (TargetHandler.IsValid())
	{
		return TargetHandler->OnDragDrop(Window);
	}

	return EDropEffect::None;
}

bool FProxyMessageHandler::OnWindowAction(const TSharedRef< FGenericWindow >& Window, const EWindowAction::Type InActionType)
{
	if (TargetHandler.IsValid())
	{
		return TargetHandler->OnWindowAction(Window, InActionType);
	}

	return true;
}
