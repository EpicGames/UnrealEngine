// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GenericPlatform/GenericApplicationMessageHandler.h"

/*
	Class that can act as a proxy for places where FGenericApplicationMessageHandler is used.
*/
class FProxyMessageHandler : public FGenericApplicationMessageHandler
{
	
public:

	FProxyMessageHandler() = default;

	FProxyMessageHandler(const TSharedPtr<FGenericApplicationMessageHandler>& InTargetHandler);

	virtual ~FProxyMessageHandler() override;

	void SetTargetHandler(const TSharedPtr<FGenericApplicationMessageHandler>& InTargetHandler);

	const TSharedPtr<FGenericApplicationMessageHandler> GetTargetHandler() const;

	virtual bool ShouldProcessUserInputMessages(const TSharedPtr< FGenericWindow >& PlatformWindow) const override;

	virtual bool OnKeyChar(const TCHAR Character, const bool IsRepeat) override;

	virtual bool OnKeyDown(const int32 KeyCode, const uint32 CharacterCode, const bool IsRepeat) override;

	virtual bool OnKeyUp(const int32 KeyCode, const uint32 CharacterCode, const bool IsRepeat) override;

	virtual bool OnMouseDown(const TSharedPtr< FGenericWindow >& Window, const EMouseButtons::Type Button) override;

	virtual bool OnMouseDown(const TSharedPtr< FGenericWindow >& Window, const EMouseButtons::Type Button, const FVector2D CursorPos) override;

	virtual bool OnMouseUp(const EMouseButtons::Type Button) override;

	virtual bool OnMouseUp(const EMouseButtons::Type Button, const FVector2D CursorPos) override;

	virtual bool OnMouseDoubleClick(const TSharedPtr< FGenericWindow >& Window, const EMouseButtons::Type Button) override;

	virtual bool OnMouseDoubleClick(const TSharedPtr< FGenericWindow >& Window, const EMouseButtons::Type Button, const FVector2D CursorPos) override;

	virtual bool OnMouseWheel(const float Delta) override;

	virtual bool OnMouseWheel(const float Delta, const FVector2D CursorPos) override;

	virtual bool OnMouseMove() override;

	virtual bool OnRawMouseMove(const int32 X, const int32 Y) override;

	virtual bool OnCursorSet() override;

	virtual bool OnControllerAnalog(FGamepadKeyNames::Type KeyName, FPlatformUserId PlatformUserId, FInputDeviceId InputDeviceId, float AnalogValue) override;

	virtual bool OnControllerButtonPressed(FGamepadKeyNames::Type KeyName, FPlatformUserId PlatformUserId, FInputDeviceId InputDeviceId, bool IsRepeat) override;

	virtual bool OnControllerButtonReleased(FGamepadKeyNames::Type KeyName, FPlatformUserId PlatformUserId, FInputDeviceId InputDeviceId, bool IsRepeat) override;

	virtual void OnBeginGesture() override;

	virtual bool OnTouchGesture(EGestureEvent GestureType, const FVector2D& Delta, float WheelDelta, bool bIsDirectionInvertedFromDevice) override;

	virtual void OnEndGesture() override;

	virtual bool OnTouchStarted(const TSharedPtr< FGenericWindow >& Window, const FVector2D& Location, float Force, int32 TouchIndex, int32 ControllerId) override;

	virtual bool OnTouchMoved(const FVector2D& Location, float Force, int32 TouchIndex, int32 ControllerId) override;

	virtual bool OnTouchEnded(const FVector2D& Location, int32 TouchIndex, int32 ControllerId) override;

	virtual bool OnTouchForceChanged(const FVector2D& Location, float Force, int32 TouchIndex, int32 ControllerId);

	virtual bool OnTouchFirstMove(const FVector2D& Location, float Force, int32 TouchIndex, int32 ControllerId);

	virtual void ShouldSimulateGesture(EGestureEvent Gesture, bool bEnable) override;

	virtual bool OnMotionDetected(const FVector& Tilt, const FVector& RotationRate, const FVector& Gravity, const FVector& Acceleration, int32 ControllerId) override;

	virtual bool OnSizeChanged(const TSharedRef< FGenericWindow >& Window, const int32 Width, const int32 Height, bool bWasMinimized = false) override;

	virtual void OnOSPaint(const TSharedRef<FGenericWindow>& Window) override;

	virtual FWindowSizeLimits GetSizeLimitsForWindow(const TSharedRef<FGenericWindow>& Window) const override;

	virtual void OnResizingWindow(const TSharedRef< FGenericWindow >& Window) override;

	virtual bool BeginReshapingWindow(const TSharedRef< FGenericWindow >& Window) override;

	virtual void FinishedReshapingWindow(const TSharedRef< FGenericWindow >& Window) override;

	virtual void HandleDPIScaleChanged(const TSharedRef< FGenericWindow >& Window) override;

	virtual void OnMovedWindow(const TSharedRef< FGenericWindow >& Window, const int32 X, const int32 Y) override;

	virtual bool OnWindowActivationChanged(const TSharedRef< FGenericWindow >& Window, const EWindowActivation ActivationType) override;

	virtual bool OnApplicationActivationChanged(const bool IsActive) override;

	virtual bool OnConvertibleLaptopModeChanged() override;

	virtual EWindowZone::Type GetWindowZoneForPoint(const TSharedRef< FGenericWindow >& Window, const int32 X, const int32 Y) override;

	virtual void OnWindowClose(const TSharedRef< FGenericWindow >& Window) override;

	virtual EDropEffect::Type OnDragEnterText(const TSharedRef< FGenericWindow >& Window, const FString& Text) override;

	virtual EDropEffect::Type OnDragEnterFiles(const TSharedRef< FGenericWindow >& Window, const TArray< FString >& Files) override;

	virtual EDropEffect::Type OnDragEnterExternal(const TSharedRef< FGenericWindow >& Window, const FString& Text, const TArray< FString >& Files) override;

	virtual EDropEffect::Type OnDragOver(const TSharedPtr< FGenericWindow >& Window) override;

	virtual void OnDragLeave(const TSharedPtr< FGenericWindow >& Window) override;

	virtual EDropEffect::Type OnDragDrop(const TSharedPtr< FGenericWindow >& Window) override;

	virtual bool OnWindowAction(const TSharedRef< FGenericWindow >& Window, const EWindowAction::Type InActionType) override;

protected:

	TSharedPtr<FGenericApplicationMessageHandler> TargetHandler;

};
