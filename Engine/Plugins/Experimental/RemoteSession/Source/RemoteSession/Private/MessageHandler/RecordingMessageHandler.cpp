// Copyright Epic Games, Inc. All Rights Reserved.

#include "RecordingMessageHandler.h"
#include "Engine/Engine.h"
#include "RemoteSession.h"
#include "MessageHandler/ProxyMessageHandler.h"
#include "Messages.h"
#include "Engine/GameViewportClient.h"

#include "Framework/Application/SlateApplication.h"
#include "Input/HittestGrid.h"
#include "Layout/WidgetPath.h"
#include "Slate/SceneViewport.h"
#include "Widgets/SViewport.h"
#include "Widgets/SWindow.h"

// helper to serialize out const params
template <typename S, typename T>
S& SerializeOut(S& Ar, const T& Value)
{
	T Tmp = Value;
	Ar << Tmp;
	return Ar;
}

#define BIND_PLAYBACK_HANDLER(Address, Func) \
	DispatchTable.FindOrAdd(Address).BindLambda([this](FArchive& InAr)\
	{\
		Func(InAr);\
	});\





FRecordingMessageHandler::FRecordingMessageHandler(const TSharedPtr<FGenericApplicationMessageHandler>& InTargetHandler)
	: FProxyMessageHandler(InTargetHandler)
{
	OutputWriter = nullptr;
	bConsumeInput = false;
	bIsTouching = false;
	bTryRouteTouchMessageToWidget = false;
	InputRect = FRect(EForceInit::ForceInitToZero);
	LastTouchLocation = FVector2D(EForceInit::ForceInitToZero);
    

	BIND_PLAYBACK_HANDLER(TEXT("OnKeyChar"), PlayOnKeyChar);
	BIND_PLAYBACK_HANDLER(TEXT("OnKeyUp"), PlayOnKeyUp);
	BIND_PLAYBACK_HANDLER(TEXT("OnKeyDown"), PlayOnKeyDown);

	BIND_PLAYBACK_HANDLER(TEXT("OnTouchStarted"), PlayOnTouchStarted);
	BIND_PLAYBACK_HANDLER(TEXT("OnTouchFirstMove"), PlayOnTouchFirstMove);
	BIND_PLAYBACK_HANDLER(TEXT("OnTouchMoved"), PlayOnTouchMoved);
	BIND_PLAYBACK_HANDLER(TEXT("OnTouchEnded"), PlayOnTouchEnded);
	BIND_PLAYBACK_HANDLER(TEXT("OnMotionDetected"), PlayOnMotionDetected);
	BIND_PLAYBACK_HANDLER(TEXT("OnBeginGesture"), PlayOnBeginGesture);
	BIND_PLAYBACK_HANDLER(TEXT("OnTouchGesture"), PlayOnTouchGesture);
	BIND_PLAYBACK_HANDLER(TEXT("OnEndGesture"), PlayOnEndGesture);
	BIND_PLAYBACK_HANDLER(TEXT("OnTouchForceChanged"), PlayOnTouchForceChanged);

	/** Deprecated controller handlers that take in the old int32 ControllerId */
	BIND_PLAYBACK_HANDLER(TEXT("OnControllerAnalog"), PlayOnControllerAnalog);
	BIND_PLAYBACK_HANDLER(TEXT("OnControllerButtonPressed"), PlayOnControllerButtonPressed);
	BIND_PLAYBACK_HANDLER(TEXT("OnControllerButtonReleased"), PlayOnControllerButtonReleased);

	/** New controller handlers that take in FPlatformUserId and FInputDeviceId */
	BIND_PLAYBACK_HANDLER(TEXT("OnControllerAnalogWithPlatformUser"), PlayOnControllerAnalogWithPlatformUser);
	BIND_PLAYBACK_HANDLER(TEXT("OnControllerButtonWithPlatformUser"), PlayOnControllerButtonPressedWithPlatformUser);
	BIND_PLAYBACK_HANDLER(TEXT("OnControllerButtonReleasedWithPlatformUser"), PlayOnControllerButtonReleasedWithPlatformUser);
}

#undef BIND_PLAYBACK_HANDLER

void FRecordingMessageHandler::SetRecordingHandler(IRecordingMessageHandlerWriter* InOutputWriter)
{
	OutputWriter = InOutputWriter;
}

void FRecordingMessageHandler::RecordMessage(const TCHAR* MsgName, const TArray<uint8>& Data)
{
	if (IsRecording())
	{
		OutputWriter->RecordMessage(MsgName, Data);
	}
}

void FRecordingMessageHandler::SetPlaybackWindow(TWeakPtr<SWindow> InWindow, TWeakPtr<FSceneViewport> InViewport)
{
	PlaybackWindow = InWindow;
	PlaybackViewport = InViewport;
}

void FRecordingMessageHandler::SetInputRect(const FVector2D& TopLeft, const FVector2D& Extents)
{
	InputRect = FRect(TopLeft, Extents);
}

bool FRecordingMessageHandler::ConvertToNormalizedScreenLocation(const FVector2D& InLocation, FVector2f& OutLocation)
{
	// note : following the LWC update, we do a FVector2D->FVector2f conversion here.

	FRect ClipRect = InputRect;
	FIntPoint Point = FIntPoint((int)InLocation.X, (int)InLocation.Y);

	if (ClipRect.Width == 0 || ClipRect.Height == 0)
	{
		ClipRect = FRect(FVector2D(EForceInit::ForceInitToZero), FVector2D(GEngine->GameViewport->Viewport->GetSizeXY()));
	}

	if (!ClipRect.Contains(Point))
	{
		OutLocation = FVector2f(EForceInit::ForceInitToZero);
		return false;
	}

	OutLocation = FVector2f((InLocation.X-ClipRect.X) / ClipRect.Width, (InLocation.Y - ClipRect.Y) / ClipRect.Height);

	return true;
}

FVector2D FRecordingMessageHandler::ConvertFromNormalizedScreenLocation(const FVector2f& ScreenLocation)
{
	// note : following the LWC update, we do a FVector2f->FVector2D conversion here.

	FVector2D ScreenLocation2D = FVector2D(ScreenLocation.X, ScreenLocation.Y);
	FVector2D OutVector = ScreenLocation2D;

	TSharedPtr<SWindow> GameWindow = PlaybackWindow.Pin();
	TSharedPtr<FSceneViewport> GameWidget = PlaybackViewport.Pin();
	if (GameWindow.IsValid())
	{
		FVector2D WindowOrigin = GameWindow->GetPositionInScreen();
		if (GameWidget.IsValid())
		{
			TSharedPtr<SViewport> ViewportWidget = GameWidget->GetViewportWidget().Pin();

			if (ViewportWidget.IsValid())
			{
				FGeometry InnerWindowGeometry = GameWindow->GetWindowGeometryInWindow();

				// Find the widget path relative to the window
				FArrangedChildren JustWindow(EVisibility::Visible);
				JustWindow.AddWidget(FArrangedWidget(GameWindow.ToSharedRef(), InnerWindowGeometry));

				FWidgetPath WidgetPath(GameWindow.ToSharedRef(), JustWindow);
				if (WidgetPath.ExtendPathTo(FWidgetMatcher(ViewportWidget.ToSharedRef()), EVisibility::Visible))
				{
					FArrangedWidget ArrangedWidget = WidgetPath.FindArrangedWidget(ViewportWidget.ToSharedRef()).Get(FArrangedWidget::GetNullWidget());

					FVector2D WindowClientOffset = ArrangedWidget.Geometry.GetAbsolutePosition();
					FVector2D WindowClientSize = ArrangedWidget.Geometry.GetAbsoluteSize();

					OutVector = WindowOrigin + WindowClientOffset + (ScreenLocation2D * WindowClientSize);

				}
			}
		}
		else
		{
			FVector2D SizeInScreen = GameWindow->GetSizeInScreen();
			OutVector = SizeInScreen * ScreenLocation2D;
			//OutVector = GameWindow->GetLocalToScreenTransform().TransformPoint(ScreenLocation);
		}
	}

	return OutVector;
}


bool FRecordingMessageHandler::PlayMessage(const TCHAR* Message, TArray<uint8> Data)
{
	FRecordedMessageDispatch* Dispatch = DispatchTable.Find(Message);

	if (Dispatch != nullptr)
	{
		FScopeLock Lock(&MessagesCriticalSection);
		FDelayPlayMessage& DelayMessage = DelayMessages.AddDefaulted_GetRef();
		DelayMessage.Dispatch = Dispatch;
		DelayMessage.Data = MoveTemp(Data);
	}
	else
	{
		UE_LOGF(LogRemoteSession, Warning, "No Playback Handler registered for message %ls", Message);
	}

	return true;
}

void FRecordingMessageHandler::Tick(const float DeltaTime)
{
	FScopeLock Lock(&MessagesCriticalSection);
	if (DelayMessages.Num())
	{
		bool WasBlocking = IsConsumingInput();
		if (WasBlocking)
		{
			SetConsumeInput(false);
		}

		for (FDelayPlayMessage& Message : DelayMessages)
		{
			FMemoryReader Ar(Message.Data);
			Message.Dispatch->ExecuteIfBound(Ar);
		}

		SetConsumeInput(WasBlocking);
		DelayMessages.Reset();
	}
}

bool FRecordingMessageHandler::OnKeyChar(const TCHAR Character, const bool IsRepeat)
{
	if (IsRecording())
	{
		TwoParamMsg<TCHAR, bool> Msg(Character, IsRepeat);
		RecordMessage(TEXT("OnKeyChar"), Msg.AsData());
	}

	if (bConsumeInput)
	{
		return true;
	}	

	return FProxyMessageHandler::OnKeyChar(Character, IsRepeat);
}

void FRecordingMessageHandler::PlayOnKeyChar(FArchive& Ar)
{
	TwoParamMsg<TCHAR, bool> Msg(Ar);
	OnKeyChar(Msg.Param1, Msg.Param2);
}

bool FRecordingMessageHandler::OnKeyDown(const int32 KeyCode, const uint32 CharacterCode, const bool IsRepeat)
{
	if (IsRecording())
	{
		ThreeParamMsg<int32, TCHAR, bool> Msg(KeyCode, CharacterCode, IsRepeat);
		RecordMessage(TEXT("OnKeyDown"), Msg.AsData());
	}

	if (bConsumeInput)
	{
		return true;
	}

	return FProxyMessageHandler::OnKeyDown(KeyCode, CharacterCode, IsRepeat);
}

void FRecordingMessageHandler::PlayOnKeyDown(FArchive& Ar)
{
	ThreeParamMsg<int32, TCHAR, bool> Msg(Ar);
	OnKeyDown(Msg.Param1, Msg.Param2, Msg.Param3);
}


bool FRecordingMessageHandler::OnKeyUp(const int32 KeyCode, const uint32 CharacterCode, const bool IsRepeat)
{
	if (IsRecording())
	{
		ThreeParamMsg<int32, TCHAR, bool> Msg(KeyCode, CharacterCode, IsRepeat);
		RecordMessage(TEXT("OnKeyUp"), Msg.AsData());
	}

	if (bConsumeInput)
	{
		return true;
	}

	return FProxyMessageHandler::OnKeyUp(KeyCode, CharacterCode, IsRepeat);
}

void FRecordingMessageHandler::PlayOnKeyUp(FArchive& Ar)
{
	ThreeParamMsg<int32, TCHAR, bool> Msg(Ar);
	OnKeyUp(Msg.Param1, Msg.Param2, Msg.Param3);
}

bool FRecordingMessageHandler::OnTouchStarted(const TSharedPtr< FGenericWindow >& Window, const FVector2D& Location, float Force, int32 TouchIndex, int32 ControllerId)
{
	if (IsRecording())
	{
		FVector2f Normalized;

		if (ConvertToNormalizedScreenLocation(Location, Normalized))
		{
			// note - force is serialized last for backwards compat - force was introduced in 4.20
			FourParamMsg<FVector2f, int32, int32, float> Msg(Normalized, TouchIndex, ControllerId, Force);
			RecordMessage(TEXT("OnTouchStarted"), Msg.AsData());
		}
	}

	bool bRouteMessageResult = true;
	if (bTryRouteTouchMessageToWidget)
	{
		FWidgetPath WidgetPath = FindRoutingMessageWidget(Location);
		if (WidgetPath.IsValid())
		{
			FScopedSwitchWorldHack SwitchWorld(WidgetPath);
			FPointerEvent PointerEvent(ControllerId, TouchIndex, Location, Location, Force, true);
			bRouteMessageResult = FSlateApplication::Get().RoutePointerDownEvent(WidgetPath, PointerEvent).IsEventHandled();
			if (!bRouteMessageResult && OnRouteTouchDownToWidgetFailedDelegate.IsBound())
			{
				OnRouteTouchDownToWidgetFailedDelegate.Broadcast(Location);
			}
		}
	}

	bIsTouching = true;
	LastTouchLocation = Location;

	if (bConsumeInput)
	{
		return true;
	}
	if (bTryRouteTouchMessageToWidget)
	{
		return bRouteMessageResult;
	}

	return FProxyMessageHandler::OnTouchStarted(Window, Location, Force, TouchIndex, ControllerId);
}

void FRecordingMessageHandler::PlayOnTouchStarted(FArchive& Ar)
{
	// LWC FVector2D are now doubles, but the OSC clients will sent floats. So we deserialize as floats
	// then create LWC version via ConvertFromNormalizedScreenLocation
	FourParamMsg<FVector2f, int32, int32, float > Msg(Ar);
	FVector2D ScreenLocation = ConvertFromNormalizedScreenLocation(Msg.Param1);

	TSharedPtr<FGenericWindow> Window;

	if (TSharedPtr<SWindow> PlaybackWindowPinned = PlaybackWindow.Pin())
	{
		Window = PlaybackWindowPinned->GetNativeWindow();
	}

	// note - force is serialized last for backwards compat - force was introduced in 4.20
	OnTouchStarted(Window, ScreenLocation, Msg.Param4, Msg.Param2, Msg.Param3 );
}

bool FRecordingMessageHandler::OnTouchMoved(const FVector2D& Location, float Force, int32 TouchIndex, int32 ControllerId)
{
	if (IsRecording())
	{
		FVector2f Normalized;

		if (ConvertToNormalizedScreenLocation(Location, Normalized))
		{
			// note - force is serialized last for backwards compat - force was introduced in 4.20
			FourParamMsg<FVector2f, int32, int32, float> Msg(Normalized, TouchIndex, ControllerId, Force);
			OutputWriter->RecordMessage(TEXT("OnTouchMoved"), Msg.AsData());
		}
	}
	
	bool bRouteMessageResult = true;
	if (bTryRouteTouchMessageToWidget)
	{
		FWidgetPath WidgetPath = FindRoutingMessageWidget(Location);
		if (WidgetPath.IsValid())
		{
			FScopedSwitchWorldHack SwitchWorld(WidgetPath);
			FPointerEvent PointerEvent(ControllerId, TouchIndex, Location, LastTouchLocation, Force, true);
			bool bIsSynthetic = false;
			bRouteMessageResult = FSlateApplication::Get().RoutePointerMoveEvent(WidgetPath, PointerEvent, bIsSynthetic);
		}
	}

	bIsTouching = true;
	LastTouchLocation = Location;

	if (bConsumeInput)
	{
		return true;
	}
	if (bTryRouteTouchMessageToWidget)
	{
		return bRouteMessageResult;
	}

	return FProxyMessageHandler::OnTouchMoved(Location, Force, TouchIndex, ControllerId);
}

void FRecordingMessageHandler::PlayOnTouchMoved(FArchive& Ar)
{
	// LWC FVector2D are now doubles, but the OSC clients will sent floats. So we deserialize as floats
	// then create LWC version via ConvertFromNormalizedScreenLocation
	FourParamMsg<FVector2f, int32, int32, float > Msg(Ar);
	FVector2D ScreenLocation = ConvertFromNormalizedScreenLocation(Msg.Param1);
	// note - force is serialized last for backwards compat - force was introduced in 4.20
	OnTouchMoved(ScreenLocation, Msg.Param4, Msg.Param2, Msg.Param3);
}

bool FRecordingMessageHandler::OnTouchEnded(const FVector2D& Location, int32 TouchIndex, int32 ControllerId)
{
	if (IsRecording())
	{
		FVector2f Normalized;

		// if outside our bounds, end the touch where it left
		if (ConvertToNormalizedScreenLocation(Location, Normalized) == false)
		{
			ConvertToNormalizedScreenLocation(LastTouchLocation, Normalized);
		}
		
		ThreeParamMsg<FVector2f, int32, int32> Msg(Normalized, TouchIndex, ControllerId);
		OutputWriter->RecordMessage(TEXT("OnTouchEnded"), Msg.AsData());
	}

	bool bRouteMessageResult = true;
	if (bTryRouteTouchMessageToWidget)
	{
		FWidgetPath WidgetPath = FindRoutingMessageWidget(Location);
		if (WidgetPath.IsValid())
		{
			FScopedSwitchWorldHack SwitchWorld(WidgetPath);
			FPointerEvent PointerEvent(ControllerId, TouchIndex, Location, Location, 0.0f, true);
			bRouteMessageResult = FSlateApplication::Get().RoutePointerUpEvent(WidgetPath, PointerEvent).IsEventHandled();
		}
	}

	bIsTouching = false;

	if (bConsumeInput)
	{
		return true;
	}
	if (bTryRouteTouchMessageToWidget)
	{
		return bRouteMessageResult;
	}

	return FProxyMessageHandler::OnTouchEnded(Location, TouchIndex, ControllerId);
}


void FRecordingMessageHandler::PlayOnTouchEnded(FArchive& Ar)
{
	// LWC FVector2D are now doubles, but the OSC clients will sent floats. So we deserialize as floats
	// then create LWC version via ConvertFromNormalizedScreenLocation
	ThreeParamMsg<FVector2f, int32, int32 > Msg(Ar);
	FVector2D ScreenLocation = ConvertFromNormalizedScreenLocation(Msg.Param1);
	OnTouchEnded(ScreenLocation, Msg.Param2, Msg.Param3);
}

bool FRecordingMessageHandler::OnTouchForceChanged(const FVector2D& Location, float Force, int32 TouchIndex, int32 ControllerId)
{
	if (IsRecording())
	{
		FVector2f Normalized;

		if (ConvertToNormalizedScreenLocation(Location, Normalized))
		{
			// note - force is serialized last for backwards compat - force was introduced in 4.20
			FourParamMsg<FVector2f, int32, int32, float> Msg(Normalized, TouchIndex, ControllerId, Force);
			OutputWriter->RecordMessage(TEXT("OnTouchForceChanged"), Msg.AsData());
		}
	}

	bool bRouteMessageResult = true;
	if (bTryRouteTouchMessageToWidget)
	{
		FWidgetPath WidgetPath = FindRoutingMessageWidget(Location);
		if (WidgetPath.IsValid())
		{
			FScopedSwitchWorldHack SwitchWorld(WidgetPath);
			FPointerEvent PointerEvent(ControllerId, TouchIndex, Location, Location, Force, true, true, false);
			bool bIsSynthetic = false;
			bRouteMessageResult = FSlateApplication::Get().RoutePointerMoveEvent(WidgetPath, PointerEvent, bIsSynthetic);
			return true;
		}
	}

	bIsTouching = true;
	LastTouchLocation = Location;

	if (bConsumeInput)
	{
		return true;
	}
	if (bTryRouteTouchMessageToWidget)
	{
		return bRouteMessageResult;
	}

	return FProxyMessageHandler::OnTouchForceChanged(Location, Force, TouchIndex, ControllerId);
}

void FRecordingMessageHandler::PlayOnTouchForceChanged(FArchive& Ar)
{
	// LWC FVector2D are now doubles, but the OSC clients will sent floats. So we deserialize as floats
	// then create LWC version via ConvertFromNormalizedScreenLocation
	FourParamMsg<FVector2f, int32, int32, float > Msg(Ar);
	FVector2D ScreenLocation = ConvertFromNormalizedScreenLocation(Msg.Param1);
	OnTouchForceChanged(ScreenLocation, Msg.Param4, Msg.Param2, Msg.Param3);
}

bool FRecordingMessageHandler::OnTouchFirstMove(const FVector2D& Location, float Force, int32 TouchIndex, int32 ControllerId)
{
	if (IsRecording())
	{
		FVector2f Normalized;

		if (ConvertToNormalizedScreenLocation(Location, Normalized))
		{
			// note - force is serialized last for backwards compat - force was introduced in 4.20
			FourParamMsg<FVector2f, int32, int32, float> Msg(Normalized, TouchIndex, ControllerId, Force);
			OutputWriter->RecordMessage(TEXT("OnTouchFirstMove"), Msg.AsData());
		}
	}

	bool bRouteMessageResult = true;
	if (bTryRouteTouchMessageToWidget)
	{
		FWidgetPath WidgetPath = FindRoutingMessageWidget(Location);
		if (WidgetPath.IsValid())
		{
			FScopedSwitchWorldHack SwitchWorld(WidgetPath);
			FPointerEvent PointerEvent(ControllerId, TouchIndex, Location, LastTouchLocation, Force, true, false, true);
			bool bIsSynthetic = false;
			bRouteMessageResult = FSlateApplication::Get().RoutePointerMoveEvent(WidgetPath, PointerEvent, bIsSynthetic);
		}
	}

	bIsTouching = true;
	LastTouchLocation = Location;

	if (bConsumeInput)
	{
		return true;
	}
	if (bTryRouteTouchMessageToWidget)
	{
		return bRouteMessageResult;
	}

	return FProxyMessageHandler::OnTouchFirstMove(Location, Force, TouchIndex, ControllerId);
}

void FRecordingMessageHandler::PlayOnTouchFirstMove(FArchive& Ar)
{
	// LWC FVector2D are now doubles, but the OSC clients will sent floats. So we deserialize as floats
	// then create the LWC version via ConvertFromNormalizedScreenLocation
	FourParamMsg<FVector2f, int32, int32, float > Msg(Ar);
	FVector2D ScreenLocation = ConvertFromNormalizedScreenLocation(Msg.Param1);
	// note - force is serialized last for backwards compat - force was introduced in 4.20
	OnTouchFirstMove(ScreenLocation, Msg.Param4, Msg.Param2, Msg.Param3);
}


void FRecordingMessageHandler::OnBeginGesture()
{
	if (IsRecording())
	{
		NoParamMsg Msg;
		OutputWriter->RecordMessage(TEXT("OnBeginGesture"), Msg.AsData());
	}

	if (bConsumeInput)
	{
		return;
	}

	FProxyMessageHandler::OnBeginGesture();
}

void FRecordingMessageHandler::PlayOnBeginGesture(FArchive& Ar)
{
	OnBeginGesture();
}

bool FRecordingMessageHandler::OnTouchGesture(EGestureEvent GestureType, const FVector2D& Delta, float WheelDelta, bool bIsDirectionInvertedFromDevice)
{
	if (IsRecording())
	{
		// Touch Delta is FVector2D, but host is parsing FVector2f, so send float to stay compatible with previous versions.
		FourParamMsg<uint32, FVector2f, float, bool> Msg(
			(uint32)GestureType,
			FVector2f(Delta.X, Delta.Y),
			WheelDelta,
			bIsDirectionInvertedFromDevice);
		OutputWriter->RecordMessage(TEXT("OnTouchGesture"), Msg.AsData());
	}

	if (bConsumeInput)
	{
		return true;
	}

	return FProxyMessageHandler::OnTouchGesture(GestureType, Delta, WheelDelta, bIsDirectionInvertedFromDevice);
}

void FRecordingMessageHandler::PlayOnTouchGesture(FArchive& Ar)
{
	FourParamMsg<uint32, FVector2f, float, bool> Msg(Ar);
	OnTouchGesture((EGestureEvent)Msg.Param1, FVector2D(Msg.Param2.X, Msg.Param2.Y), Msg.Param3, Msg.Param4);
}

void FRecordingMessageHandler::OnEndGesture()
{
	if (IsRecording())
	{
		NoParamMsg Msg;
		OutputWriter->RecordMessage(TEXT("OnEndGesture"), Msg.AsData());
	}

	if (bConsumeInput)
	{
		return;
	}

	FProxyMessageHandler::OnEndGesture();
}

void FRecordingMessageHandler::PlayOnEndGesture(FArchive& Ar)
{
	OnEndGesture();
}

bool FRecordingMessageHandler::OnMotionDetected(const FVector& Tilt, const FVector& RotationRate, const FVector& Gravity, const FVector& Acceleration, int32 ControllerId)
{
	if (IsRecording())
	{
		// LWC FVector3D are now doubles, but either legacy client or legacy host might use floats. So we Serialize as floats then create LWC versions in deserialization.
		FiveParamMsg<FVector3f, FVector3f, FVector3f, FVector3f, int32> Msg(
			FVector3f(Tilt.X, Tilt.Y, Tilt.Z),
			FVector3f(RotationRate.X, RotationRate.Y, RotationRate.Z),
			FVector3f(Gravity.X, Gravity.Y, Gravity.Z),
			FVector3f(Acceleration.X, Acceleration.Y, Acceleration.Z),
			ControllerId);
		OutputWriter->RecordMessage(TEXT("OnMotionDetected"), Msg.AsData());
	}

	if (bConsumeInput)
	{
		return true;
	}

	return FProxyMessageHandler::OnMotionDetected(Tilt, RotationRate, Gravity, Acceleration, ControllerId);
}

void FRecordingMessageHandler::PlayOnMotionDetected(FArchive& Ar)
{
	// LWC FVector3D are now doubles, but the OSC clients will sent floats. So we deserialize as floats then create LWC versions below.
	FiveParamMsg<FVector3f, FVector3f, FVector3f, FVector3f, int32 > Msg(Ar);
	OnMotionDetected(
		FVector(Msg.Param1.X, Msg.Param1.Y, Msg.Param1.Z), 
		FVector(Msg.Param2.X, Msg.Param2.Y, Msg.Param2.Z),
		FVector(Msg.Param3.X, Msg.Param3.Y, Msg.Param3.Z),
		FVector(Msg.Param4.X, Msg.Param4.Y, Msg.Param4.Z),
		Msg.Param5);
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS

void FRecordingMessageHandler::PlayOnControllerAnalog(FArchive& Ar)
{
	ThreeParamMsg<FString, int32, float > Msg(Ar);
	
	const int32 LegacyControllerId = Msg.Param2;
	FPlatformUserId PlatformUser = PLATFORMUSERID_NONE;
	FInputDeviceId DeviceId = INPUTDEVICEID_NONE;
	IPlatformInputDeviceMapper::Get().RemapControllerIdToPlatformUserAndDevice(LegacyControllerId, PlatformUser, DeviceId);
	
	OnControllerAnalog(FName(*Msg.Param1), PlatformUser, DeviceId, Msg.Param3);
}

void FRecordingMessageHandler::PlayOnControllerButtonPressed(FArchive& Ar)
{
	ThreeParamMsg<FString, int32, bool > Msg(Ar);

	const int32 LegacyControllerId = Msg.Param2;
	FPlatformUserId PlatformUser = PLATFORMUSERID_NONE;
	FInputDeviceId DeviceId = INPUTDEVICEID_NONE;
	IPlatformInputDeviceMapper::Get().RemapControllerIdToPlatformUserAndDevice(LegacyControllerId, PlatformUser, DeviceId);
	
	OnControllerButtonPressed(FName(*Msg.Param1), PlatformUser, DeviceId, Msg.Param3);
}

void FRecordingMessageHandler::PlayOnControllerButtonReleased(FArchive& Ar)
{
	ThreeParamMsg<FString, int32, bool > Msg(Ar);

	const int32 LegacyControllerId = Msg.Param2;
	FPlatformUserId PlatformUser = PLATFORMUSERID_NONE;
	FInputDeviceId DeviceId = INPUTDEVICEID_NONE;
	IPlatformInputDeviceMapper::Get().RemapControllerIdToPlatformUserAndDevice(LegacyControllerId, PlatformUser, DeviceId);
	
	OnControllerButtonReleased(FName(*Msg.Param1), PlatformUser, DeviceId, Msg.Param3);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

bool FRecordingMessageHandler::OnControllerAnalog(FGamepadKeyNames::Type KeyName, FPlatformUserId PlatformUserId, FInputDeviceId InputDeviceId, float AnalogValue)
{
	if (IsRecording())
	{
		FourParamMsg<FString, int32, int32, float> Msg(KeyName.ToString(), PlatformUserId.GetInternalId(), InputDeviceId.GetId(), AnalogValue);
		RecordMessage(TEXT("OnControllerAnalogWithPlatformUser"), Msg.AsData());
	}

	if (bConsumeInput)
	{
		return true;
	}
	
	return FProxyMessageHandler::OnControllerAnalog(KeyName, PlatformUserId, InputDeviceId, AnalogValue);
}

void FRecordingMessageHandler::PlayOnControllerAnalogWithPlatformUser(FArchive& Ar)
{
	FourParamMsg<FString, int32, int32, float> Msg(Ar);
	OnControllerAnalog(FName(*Msg.Param1), FPlatformUserId::CreateFromInternalId(Msg.Param2), FInputDeviceId::CreateFromInternalId(Msg.Param3), Msg.Param4);
}

bool FRecordingMessageHandler::OnControllerButtonPressed(FGamepadKeyNames::Type KeyName, FPlatformUserId PlatformUserId, FInputDeviceId InputDeviceId, bool IsRepeat)
{
	if (IsRecording())
	{
		FourParamMsg<FString, int32, int32, bool> Msg(KeyName.ToString(), PlatformUserId.GetInternalId(), InputDeviceId.GetId(), IsRepeat);
		RecordMessage(TEXT("OnControllerButtonPressedWithPlatformUser"), Msg.AsData());
	}

	if (bConsumeInput)
	{
		return true;
	}

	return FProxyMessageHandler::OnControllerButtonPressed(KeyName, PlatformUserId, InputDeviceId, IsRepeat);
}

void FRecordingMessageHandler::PlayOnControllerButtonPressedWithPlatformUser(FArchive& Ar)
{
	FourParamMsg<FString, int32, int32, bool> Msg(Ar);
	OnControllerButtonPressed(FName(*Msg.Param1), FPlatformUserId::CreateFromInternalId(Msg.Param2), FInputDeviceId::CreateFromInternalId(Msg.Param3), Msg.Param4);
}

bool FRecordingMessageHandler::OnControllerButtonReleased(FGamepadKeyNames::Type KeyName, FPlatformUserId PlatformUserId, FInputDeviceId InputDeviceId, bool IsRepeat)
{
	if (IsRecording())
	{
		FourParamMsg<FString, int32, int32, bool> Msg(KeyName.ToString(), PlatformUserId.GetInternalId(), InputDeviceId.GetId(), IsRepeat);
		RecordMessage(TEXT("OnControllerButtonReleasedWithPlatformUser"), Msg.AsData());
	}

	if (bConsumeInput)
	{
		return true;
	}

	return FProxyMessageHandler::OnControllerButtonReleased(KeyName, PlatformUserId, InputDeviceId, IsRepeat);
}

void FRecordingMessageHandler::PlayOnControllerButtonReleasedWithPlatformUser(FArchive& Ar)
{
	FourParamMsg<FString, int32, int32, bool> Msg(Ar);
	OnControllerButtonReleased(FName(*Msg.Param1), FPlatformUserId::CreateFromInternalId(Msg.Param2), FInputDeviceId::CreateFromInternalId(Msg.Param3), Msg.Param4);
}

FWidgetPath FRecordingMessageHandler::FindRoutingMessageWidget(const FVector2D& Location) const
{
	if (TSharedPtr<SWindow> PlaybackWindowPinned = PlaybackWindow.Pin())
	{
		if (PlaybackWindowPinned->AcceptsInput())
		{
			bool bIgnoreEnabledStatus = false;
			TArray<FWidgetAndPointer> WidgetsAndCursors = PlaybackWindowPinned->GetHittestGrid().GetBubblePath(Location, FSlateApplication::Get().GetCursorRadius(), bIgnoreEnabledStatus);
			return FWidgetPath(MoveTemp(WidgetsAndCursors));
		}
	}
	return FWidgetPath();
}
