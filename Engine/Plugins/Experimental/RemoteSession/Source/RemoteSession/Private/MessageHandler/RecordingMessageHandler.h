// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ProxyMessageHandler.h"

class FWidgetPath;

class SWindow;
class FSceneViewport;

class IRecordingMessageHandlerWriter
{
public:

	virtual void RecordMessage(const TCHAR* MsgName, const TArray<uint8>& Data) = 0;
};

DECLARE_DELEGATE_OneParam(FRecordedMessageDispatch, FArchive&);
DECLARE_MULTICAST_DELEGATE_OneParam(FOnRouteTouchDownToWidgetFailedDelegate, const FVector2D& /*ViewportPosition*/);

class FRecordingMessageHandler : public FProxyMessageHandler, public TSharedFromThis<FRecordingMessageHandler>
{
	struct FRect
	{
		FRect() = default;

		FRect(EForceInit)
			: X(0.0f)
			, Y(0.0f)
			, Width(0.0f)
			, Height(0.0f)
		{}

		FRect(float InX, float InY, float InWidth, float InHeight)
			: X(InX)
			, Y(InY)
			, Width(InWidth)
			, Height(InHeight)
		{}

		FRect(const FVector2D& Point, const FVector2D& Extents)
			: X(Point.X)
			, Y(Point.Y)
			, Width(Extents.X)
			, Height(Extents.Y)
		{}

		FORCEINLINE bool Contains(FVector2D P) const
		{
			return P.X >= X && P.X < (X+Width) && P.Y >= Y && P.Y < (Y+Height);
		}

		float	X;
		float	Y;
		float	Width;
		float	Height;
	};
public:

	FRecordingMessageHandler(const TSharedPtr<FGenericApplicationMessageHandler>& InTargetHandler);

	void Tick(const float DeltaTime);

	bool PlayMessage(const TCHAR* Message, TArray<uint8> Data);

	void SetRecordingHandler(IRecordingMessageHandlerWriter* InOutputWriter);

	/**
	 * Do not pass input to the target handler
	 */
	void SetConsumeInput(bool bConsume) { bConsumeInput = bConsume; }

	/**
	 * Returns true/false depending on whether input is blocked
	 */
	bool IsConsumingInput() const { return bConsumeInput; }

	/**
	 * Returns true/false depending on whether we are recording (RecordingHandler is != null)
	 */
	bool IsRecording() const {	return OutputWriter != nullptr; }

	/**
	 * Set the window to which the input should be forward to
	 */
	void SetPlaybackWindow(TWeakPtr<SWindow> InWindow, TWeakPtr<FSceneViewport> InViewport);

	void SetInputRect(const FVector2D& TopLeft, const FVector2D& Extents);

	/*
	 * Send any received touch event from the client to the widget directly bypassing the message handler
	 */
	void TryRouteTouchMessageToWidget(bool bInRouteMessageToWidget) { bTryRouteTouchMessageToWidget = bInRouteMessageToWidget; }

	FOnRouteTouchDownToWidgetFailedDelegate& GetOnRouteTouchDownToWidgetFailedDelegate() { return OnRouteTouchDownToWidgetFailedDelegate; }

public:

	/* FRecordingMessageHandler does not support sending PlatformUserId */
	virtual bool ShouldUsePlatformUserId() const override
	{
		return false;
	}

	/**
	 * Keyboard handling
	 */
	virtual bool OnKeyChar(const TCHAR Character, const bool IsRepeat) override;
	virtual bool OnKeyDown(const int32 KeyCode, const uint32 CharacterCode, const bool IsRepeat) override;
	virtual bool OnKeyUp(const int32 KeyCode, const uint32 CharacterCode, const bool IsRepeat) override;

	/**
	 * High-level gesture events
	 */
	virtual void OnBeginGesture() override;
	virtual bool OnTouchGesture(EGestureEvent GestureType, const FVector2D& Delta, float WheelDelta, bool bIsDirectionInvertedFromDevice) override;
	virtual void OnEndGesture() override;

	/**
	 * Raw touch events
	 */
	virtual bool OnTouchStarted(const TSharedPtr< FGenericWindow >& Window, const FVector2D& Location, float Force, int32 TouchIndex, int32 ControllerId) override;
	virtual bool OnTouchMoved(const FVector2D& Location, float Force, int32 TouchIndex, int32 ControllerId) override;
	virtual bool OnTouchEnded(const FVector2D& Location, int32 TouchIndex, int32 ControllerId) override;
	virtual bool OnTouchForceChanged(const FVector2D& Location, float Force, int32 TouchIndex, int32 ControllerId) override;
	virtual bool OnTouchFirstMove(const FVector2D& Location, float Force, int32 TouchIndex, int32 ControllerId) override;
	virtual bool OnMotionDetected(const FVector& Tilt, const FVector& RotationRate, const FVector& Gravity, const FVector& Acceleration, int32 ControllerId) override;

	/**
	 * Controller handling
	 */
	virtual bool OnControllerAnalog(FGamepadKeyNames::Type KeyName, FPlatformUserId PlatformUserId, FInputDeviceId InputDeviceId, float AnalogValue) override;
	virtual bool OnControllerButtonPressed(FGamepadKeyNames::Type KeyName, FPlatformUserId PlatformUserId, FInputDeviceId InputDeviceId, bool IsRepeat) override;
	virtual bool OnControllerButtonReleased(FGamepadKeyNames::Type KeyName, FPlatformUserId PlatformUserId, FInputDeviceId InputDeviceId, bool IsRepeat) override;
	
	
protected:

	bool ConvertToNormalizedScreenLocation(const FVector2D& InLocation, FVector2f& OutLocation);
	FVector2D ConvertFromNormalizedScreenLocation(const FVector2f& ScreenLocation);

	void RecordMessage(const TCHAR* MsgName, const TArray<uint8>& Data);

	virtual void PlayOnKeyChar(FArchive& Ar);
	virtual void PlayOnKeyDown(FArchive& Ar);
	virtual void PlayOnKeyUp(FArchive& Ar);

	virtual void PlayOnBeginGesture(FArchive& Ar);
	virtual void PlayOnTouchGesture(FArchive& Ar);
	virtual void PlayOnEndGesture(FArchive& Ar);

	virtual void PlayOnTouchStarted(FArchive& Ar);
	virtual void PlayOnTouchMoved(FArchive& Ar);
	virtual void PlayOnTouchEnded(FArchive& Ar);
	virtual void PlayOnTouchForceChanged(FArchive& Ar);
	virtual void PlayOnTouchFirstMove(FArchive& Ar);
	virtual void PlayOnMotionDetected(FArchive& Ar);

	virtual void PlayOnControllerAnalog(FArchive& Ar);
	virtual void PlayOnControllerButtonPressed(FArchive& Ar);
	virtual void PlayOnControllerButtonReleased(FArchive& Ar);
	
	virtual void PlayOnControllerAnalogWithPlatformUser(FArchive& Ar);
	virtual void PlayOnControllerButtonPressedWithPlatformUser(FArchive& Ar);
	virtual void PlayOnControllerButtonReleasedWithPlatformUser(FArchive& Ar);

	FWidgetPath FindRoutingMessageWidget(const FVector2D& Location) const;

	struct FDelayPlayMessage
	{
		FRecordedMessageDispatch* Dispatch;
		TArray<uint8> Data;
	};

	IRecordingMessageHandlerWriter*		OutputWriter;
	bool								bConsumeInput;
	TWeakPtr<SWindow>					PlaybackWindow;
	TWeakPtr<FSceneViewport>			PlaybackViewport;

	TMap<FString, FRecordedMessageDispatch> DispatchTable;

	mutable FCriticalSection		MessagesCriticalSection;
	TArray<FDelayPlayMessage>		DelayMessages;

	FRect								InputRect;
    FVector2D							LastTouchLocation;
    bool								bIsTouching;
	bool								bTryRouteTouchMessageToWidget;

	FOnRouteTouchDownToWidgetFailedDelegate OnRouteTouchDownToWidgetFailedDelegate;
};
