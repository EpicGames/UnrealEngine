// Copyright Epic Games, Inc. All Rights Reserved.

#include "RemoteSessionClient.h"
#include "BackChannel/Protocol/OSC/BackChannelOSCConnection.h"
#include "BackChannel/Transport/IBackChannelTransport.h"
#include "BackChannel/Protocol/OSC/BackChannelOSCMessage.h"
#include "BackChannel/Transport/IBackChannelSocketConnection.h"
#include "RemoteSession.h"
#include "Async/Async.h"

DECLARE_CYCLE_STAT(TEXT("RSClientTick"), STAT_RDClientTick, STATGROUP_Game);

FRemoteSessionClient::FRemoteSessionClient(const TCHAR* InHostAddress)
{
	HostAddress = InHostAddress;
	ConnectionAttemptTimer = FLT_MAX;		// attempt a connection asap
	TimeConnectionAttemptStarted = 0;
    ConnectionTimeout = 5;

	IsConnecting = false;

	// Detect whether the address already includes a port
	bool bHasPort = false;
	if (HostAddress.StartsWith(TEXT("[")))
	{
		// IPv6 bracket notation — port follows the closing bracket, e.g. [::1]:2049
		int32 CloseBracket;
		if (HostAddress.FindChar(TEXT(']'), CloseBracket))
		{
			bHasPort = HostAddress.Mid(CloseBracket + 1).StartsWith(TEXT(":"));
		}
	}
	else
	{
		// A single colon means IPv4/hostname with port (e.g. "192.168.1.1:2049").
		// Multiple colons means bare IPv6 without port (e.g. "::1", "fd00::1").
		int32 FirstColon = HostAddress.Find(TEXT(":"), ESearchCase::IgnoreCase, ESearchDir::FromStart);
		int32 LastColon = HostAddress.Find(TEXT(":"), ESearchCase::IgnoreCase, ESearchDir::FromEnd);
		bHasPort = (FirstColon != INDEX_NONE && FirstColon == LastColon);
	}

	if (!bHasPort)
	{
		if (HostAddress.StartsWith(TEXT("[")))
		{
			// [::1] -> [::1]:2049
			HostAddress += FString::Printf(TEXT(":%d"), (int32)IRemoteSessionModule::kDefaultPort);
		}
		else if (HostAddress.Contains(TEXT(":")))
		{
			// Bare IPv6 like "::1" -> "[::1]:2049"
			HostAddress = FString::Printf(TEXT("[%s]:%d"), *HostAddress, (int32)IRemoteSessionModule::kDefaultPort);
		}
		else
		{
			// IPv4 or hostname -> "host:2049"
			HostAddress += FString::Printf(TEXT(":%d"), (int32)IRemoteSessionModule::kDefaultPort);
		}
	}

	// OnReceiveHello will enable PixelStreaming if supported by both host and client
	bIsUsingPixelStreaming = false;

	UE_LOGF(LogRemoteSession, Display, "Will attempt to connect to %ls..", *HostAddress);
}

FRemoteSessionClient::~FRemoteSessionClient()
{
	CloseConnections();
}

bool FRemoteSessionClient::IsConnected() const
{
	// this is to work-around the UE BSD socket implt always saying
	// things are connected for the first 5 secs...
	return FRemoteSessionRole::IsConnected() && Connection->GetPacketsReceived() > 0;
}

void FRemoteSessionClient::Tick(float DeltaTime)
{
	SCOPE_CYCLE_COUNTER(STAT_RDClientTick);

	if (IsConnected() == false)
	{
		if (IsConnecting == false && HasError()  == false)
		{
			const double TimeSinceLastAttempt = FPlatformTime::Seconds() - TimeConnectionAttemptStarted;

			if (TimeSinceLastAttempt >= 5.0)
			{
				StartConnection();
			}
		}

		if (IsConnecting)
		{
			CheckConnection();
		}
	}

	FRemoteSessionRole::Tick(DeltaTime);
}

void  FRemoteSessionClient::StartConnection()
{
	check(IsConnecting == false);

	CloseConnections();

	if (IBackChannelTransport* Transport = IBackChannelTransport::Get())
	{
		Connection = Transport->CreateConnection(IBackChannelTransport::TCP);

		if (Connection.IsValid())
		{
			if (Connection->Connect(*HostAddress))
			{
				IsConnecting = true;
				check(Connection->GetSocket());
			}
		}
	}

	TimeConnectionAttemptStarted = FPlatformTime::Seconds();
}

void FRemoteSessionClient::CheckConnection()
{
	check(IsConnected() == false && IsConnecting == true);
	check(Connection->GetSocket());

	// success indicates that our check was successful, if our connection was successful then
	// the delegate code is called
	bool Success = Connection->WaitForConnection(0, [this](auto InConnection)
	{
		CreateOSCConnection(Connection.ToSharedRef());
		
		UE_LOGF(LogRemoteSession, Log, "Connected to host at %ls", *HostAddress);

		IsConnecting = false;

		//SetReceiveInBackground(true);

		return true;
	});

	const double TimeSpentConnecting = FPlatformTime::Seconds() - TimeConnectionAttemptStarted;

	if (IsConnected() == false)
	{
		if (Success == false || TimeSpentConnecting >= ConnectionTimeout)
		{
			IsConnecting = false;
			
			FString Msg;
			
			if (TimeSpentConnecting >= ConnectionTimeout)
			{
				Msg = FString::Printf(TEXT("Timing out connection attempt after %.02f seconds"), TimeSpentConnecting);
			}
			else
			{
				Msg = TEXT("Failed to check for connection. Aborting.");
			}
			
			UE_LOGF(LogRemoteSession, Log, "%ls", *Msg);
			
			CloseConnections();
			TimeConnectionAttemptStarted = FPlatformTime::Seconds();
		}
	}
}

void FRemoteSessionClient::BindEndpoints(TBackChannelSharedPtr<IBackChannelConnection> InConnection)
{
	FRemoteSessionRole::BindEndpoints(InConnection);
		
	// both legacy and new channel messages can go to the same delegate
	auto Delegate = FBackChannelRouteDelegate::FDelegate::CreateRaw(this, &FRemoteSessionClient::OnReceiveChannelList);
	InConnection->AddRouteDelegate(kLegacyChannelSelectionEndPoint, Delegate);
	InConnection->AddRouteDelegate(kChannelListEndPoint, Delegate);

	Delegate = FBackChannelRouteDelegate::FDelegate::CreateRaw(this, &FRemoteSessionClient::OnReceiveHello);
	RegisterHelloRouteMessageDelegate(Delegate);
}

bool FRemoteSessionClient::ProcessStateChange(const ConnectionState NewState, const ConnectionState OldState)
{
	if (NewState == FRemoteSessionRole::ConnectionState::UnversionedConnection)
	{
		BindEndpoints(OSCConnection);

		// send these both. Hello will always win
		SendHello();

		SendLegacyVersionCheck();

		SetPendingState(FRemoteSessionRole::ConnectionState::EstablishingVersion);
	}

	return true;
}

void FRemoteSessionClient::AppendExtraInfoToBackChannelPacket(TBackChannelSharedPtr<IBackChannelPacket> Packet)
{
	if (Packet->GetPath() == kHelloEndPoint)
	{
		if (FModuleManager::Get().IsModuleLoaded("PixelStreaming2"))
		{
			Packet->Write(TEXT("PSVersion"), TEXT("PS2"));
		}
	}
}

void FRemoteSessionClient::OnReceiveHello(IBackChannelPacket& Message)
{
	// Need to read the message values one by one to get the correct data we need
	Message.ResetReading();

	FString VerString;
	Message.Read(TEXT("Version"), VerString);

	RemotePixelStreamingVersion.Empty();
	Message.Read(TEXT("PSVersion"), RemotePixelStreamingVersion);

	// Negotiate: use PS if host supports PS2 and we have the PS2 module
	bool bLocalPSCapable = FModuleManager::Get().IsModuleLoaded("PixelStreaming2");
	bIsUsingPixelStreaming = bLocalPSCapable && RemotePixelStreamingVersion == TEXT("PS2");

	UE_LOGF(LogRemoteSession, Log, "Negotiated streaming mode: %ls (host PSVersion: %ls, local PS2 module: %ls)",
		bIsUsingPixelStreaming ? TEXT("PixelStreaming") : TEXT("MJPEG"),
		RemotePixelStreamingVersion.IsEmpty() ? TEXT("none") : *RemotePixelStreamingVersion,
		bLocalPSCapable ? TEXT("loaded") : TEXT("not loaded"));
}

void FRemoteSessionClient::OnReceiveChannelList(IBackChannelPacket& Message)
{
	AvailableChannels.Empty();

	// #agrant todo - not safe, how to express this now?
	FBackChannelOSCMessage& OSCMessage = *static_cast<FBackChannelOSCMessage*>(&Message);
	
	int NumChannels = 0; //
	
	if (!IsLegacyConnection())
	{
		OSCMessage.Read(TEXT("ChannelCount"), NumChannels);

		if (NumChannels == 0)
		{
			UE_LOGF(LogRemoteSession, Error, "Received ChannelList messsage with no channels!");

		}
		int ChannelTags = (OSCMessage.GetArgumentCount() - 1) / 2;
		ensureMsgf(NumChannels == ChannelTags, TEXT("Channel count was %d but number of channel pairs was %d"), NumChannels, ChannelTags);
	}
	else
	{
		// need to interpret this from the argument count
		NumChannels = OSCMessage.GetArgumentCount() / 2;
	}

	for (int i = 0; i < NumChannels; i++)
	{
		FString ChannelName;
		FString ChannelModeStr;
		ERemoteSessionChannelMode ChannelMode = ERemoteSessionChannelMode::Unknown;
		
		OSCMessage.Read(TEXT("Name"), ChannelName);

		if (IsLegacyConnection())
		{
			// in legacy modes we sent an int and only had read/write. 
			int32 ChannelModeInt = 0;
			OSCMessage.Read(TEXT("Mode"), ChannelModeInt);
			ChannelMode = (ERemoteSessionChannelMode)(ChannelModeInt+1);
		}
		else
		{
			// In new protocol a string.
			OSCMessage.Read(TEXT("Mode"), ChannelModeStr);
			LexFromString(ChannelMode, *ChannelModeStr);
		}
		
		if (ChannelName.Len() && ChannelMode > ERemoteSessionChannelMode::Unknown)
		{
			UE_LOGF(LogRemoteSession, Log, "Remote host supports channel %ls with mode %ls", *ChannelName, ::LexToString((ERemoteSessionChannelMode)ChannelMode));
			AvailableChannels.Emplace(MoveTemp(ChannelName), ChannelMode);
		}
		else
		{
			UE_LOGF(LogRemoteSession, Error, "Failed to read channel from ChannelSelection message!");
		}
	}

	if (IsLegacyConnection())
	{
		// Need to create channels on the main thread
		AsyncTask(ENamedThreads::GameThread, [this] {
				CreateChannels(AvailableChannels);
			});
		UE_LOGF(LogRemoteSession, Log, "Creating all channels for legacy connection");
	}
	else
	{
		// broadcast the delegate on the main thread
		AsyncTask(ENamedThreads::GameThread, [this] {
			ReceiveChannelListDelegate.Broadcast(this, AvailableChannels);
		});
	}
}
