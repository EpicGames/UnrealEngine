// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RemoteSessionRole.h"

struct FRemoteSessionChannelInfo;

class FBackChannelOSCMessage;
class FBackChannelOSCDispatch;

class FRemoteSessionClient : public FRemoteSessionRole
{
public:

	FRemoteSessionClient(const TCHAR* InHostAddress);
	~FRemoteSessionClient();

	virtual void Tick(float DeltaTime) override;

	virtual bool IsConnected() const override;

protected:
	
	void 				StartConnection();
	void 				CheckConnection();
	
	virtual void		BindEndpoints(TBackChannelSharedPtr<IBackChannelConnection> InConnection);
	virtual bool		ProcessStateChange(const ConnectionState NewState, const ConnectionState OldState) override;

	void				OnReceiveChannelList(IBackChannelPacket& Message);

	void				RequestChannel(const FRemoteSessionChannelInfo& Info);

	void				OnReceiveHello(IBackChannelPacket& Message);
	virtual void		AppendExtraInfoToBackChannelPacket(TBackChannelSharedPtr<IBackChannelPacket> Packet) override;

	FString				HostAddress;
	
	bool				IsConnecting;
    float               ConnectionTimeout;

	double				ConnectionAttemptTimer;
	double				TimeConnectionAttemptStarted;
	
	FDelegateHandle		ChannelCallbackHandle;

	TArray<FRemoteSessionChannelInfo> AvailableChannels;
};
