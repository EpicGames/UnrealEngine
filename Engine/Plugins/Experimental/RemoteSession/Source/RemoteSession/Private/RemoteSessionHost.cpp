// Copyright Epic Games, Inc. All Rights Reserved.

#include "RemoteSessionHost.h"
#include "BackChannel/IBackChannelPacket.h"
#include "BackChannel/Transport/IBackChannelTransport.h"
#include "BackChannel/Protocol/OSC/BackChannelOSCConnection.h"
#include "Framework/Application/SlateApplication.h"
#include "BackChannel/Transport/IBackChannelSocketConnection.h"
#include "RemoteSessionModule.h"
#include "Trace/Trace.inl"
#include "Engine/Engine.h"

#define ENABLE_PIXEL_STREAMING_2 1

#if ENABLE_PIXEL_STREAMING_2
#if WITH_EDITOR
#include "IPixelStreaming2Module.h"
#include "IPixelStreaming2EditorModule.h"
#endif //WITH_EDITOR

#else //ENABLE_PIXEL_STREAMING_2

#if WITH_EDITOR
#include "IPixelStreamingModule.h"
#include "IPixelStreamingEditorModule.h"
#endif //WITH_EDITOR

#endif //ENABLE_PIXEL_STREAMING_2



namespace RemoteSessionEd
{
	static FAutoConsoleVariable SlateDragDistanceOverride(TEXT("RemoteSessionEd.SlateDragDistanceOverride"), 10.0f, TEXT("How many pixels you need to drag before a drag and drop operation starts in remote app"));
};


FRemoteSessionHost::FRemoteSessionHost(TArray<FRemoteSessionChannelInfo> InSupportedChannels)
	: HostTCPPort(0)
	, IsListenerConnected(false)
{
	SupportedChannels = InSupportedChannels;
	SavedEditorDragTriggerDistance = FSlateApplication::Get().GetDragTriggerDistance();
}

FRemoteSessionHost::~FRemoteSessionHost()
{
	// close this manually to force the thread to stop before things start to be 
	// destroyed
	if (Listener.IsValid())
	{
		Listener->Close();
	}

	CloseConnections();
}

void FRemoteSessionHost::CloseConnections()
{
	FRemoteSessionRole::CloseConnections();

	if (FSlateApplication::IsInitialized())
	{
		FSlateApplication::Get().SetDragTriggerDistance(SavedEditorDragTriggerDistance);
	}
}

void FRemoteSessionHost::SetScreenSharing(const bool bEnabled)
{
}


bool FRemoteSessionHost::StartListening(const uint16 InPort)
{
	if (Listener.IsValid())
	{
		return false;
	}

	if (IBackChannelTransport* Transport = IBackChannelTransport::Get())
	{
		Listener = Transport->CreateConnection(IBackChannelTransport::TCP);

		if (Listener->Listen(InPort) == false)
		{
			Listener = nullptr;
		}
		HostTCPPort = InPort;
	}

	return Listener.IsValid();
}

bool FRemoteSessionHost::ProcessStateChange(const ConnectionState NewState, const ConnectionState OldState)
{
	if (NewState == FRemoteSessionRole::ConnectionState::UnversionedConnection)
	{
		BindEndpoints(OSCConnection);

		// send these both. Hello will always win
		SendHello();

		SendLegacyVersionCheck();

		SetPendingState(FRemoteSessionRole::ConnectionState::EstablishingVersion);
	}
	else if (NewState == FRemoteSessionRole::ConnectionState::Connected)
	{
		ClearChannels();

		IsListenerConnected = true;

		SendChannelListToConnection();

		if (bIsUsingPixelStreaming && !RemotePixelStreamingVersion.IsEmpty())
		{
#if ENABLE_PIXEL_STREAMING_2
			if (FModuleManager::Get().IsModuleLoaded("PixelStreaming2"))
			{
				GEngine->Exec(GEngine->GetWorld(), TEXT("PixelStreaming2.WebRTC.DisableReceiveVideo 1"));
				GEngine->Exec(GEngine->GetWorld(), TEXT("PixelStreaming2.DecoupleFrameRate 1"));
				GEngine->Exec(GEngine->GetWorld(), TEXT("PixelStreaming2.WebRTC.MinBitrate 5000000"));
#if WITH_EDITOR
				if (FModuleManager::Get().IsModuleLoaded("PixelStreaming2Editor"))
				{
					bool IsStreaming = false;
					IPixelStreaming2Module::Get().ForEachStreamer([&](TSharedPtr<IPixelStreaming2Streamer> Streamer) {
						TSharedPtr<IPixelStreaming2VideoProducer> VideoProducer = Streamer->GetVideoProducer().Pin();
						if (VideoProducer.IsValid() && (VideoProducer->ToString() == TEXT("the PIE Viewport")))
						{
							if (!Streamer->IsStreaming())
							{
								Streamer->StartStreaming();
							}
						}
					});

					IPixelStreaming2EditorModule::Get().StartSignalling();
				}
#endif //WITH_EDITOR
			}
#endif //ENABLE_PIXEL_STREAMING_2
		}
	}

	return true;
}


void FRemoteSessionHost::BindEndpoints(TBackChannelSharedPtr<IBackChannelConnection> InConnection)
{
	FRemoteSessionRole::BindEndpoints(InConnection);

	auto Delegate = FBackChannelRouteDelegate::FDelegate::CreateRaw(this, &FRemoteSessionHost::OnReceiveHello);
	RegisterHelloRouteMessageDelegate(Delegate);
}


void FRemoteSessionHost::SendChannelListToConnection()
{	
	FRemoteSessionModule& RemoteSession = FModuleManager::GetModuleChecked<FRemoteSessionModule>("RemoteSession");

	TBackChannelSharedPtr<IBackChannelPacket> Packet = OSCConnection->CreatePacket();

	if (IsLegacyConnection())
	{
		Packet->SetPath(kLegacyChannelSelectionEndPoint);
	}
	else
	{
		Packet->SetPath(kChannelListEndPoint);
		Packet->Write(TEXT("ChannelCount"), SupportedChannels.Num());
	}
	

	// send these across as a name/mode pair
	for (const FRemoteSessionChannelInfo& Channel : SupportedChannels)
	{
		ERemoteSessionChannelMode ClientMode = (Channel.Mode == ERemoteSessionChannelMode::Write) ? ERemoteSessionChannelMode::Read : ERemoteSessionChannelMode::Write;

		Packet->Write(TEXT("ChannelName"), Channel.Type);

		if (IsLegacyConnection())
		{
			// legacy mode is an int where 0 = read and 1 = write
			int32 ClientInt = ClientMode == ERemoteSessionChannelMode::Read ? 0 : 1;
			Packet->Write(TEXT("ChannelMode"), ClientInt);
		}
		else
		{
			// new protocol is a string
			Packet->Write(TEXT("ChannelMode"), ::LexToString(ClientMode));
		}

		UE_LOGF(LogRemoteSession, Log, "Offering channel %ls with mode %d", *Channel.Type, int(ClientMode));
	}
	
	OSCConnection->SendPacket(Packet);

	if (IsLegacyConnection())
	{
		UE_LOGF(LogRemoteSession, Log, "Pre-creating channels for legacy connection");
		CreateChannels(SupportedChannels);
	}
}

void FRemoteSessionHost::AppendExtraInfoToBackChannelPacket(TBackChannelSharedPtr<IBackChannelPacket> Packet)
{
	if (Packet->GetPath() == kHelloEndPoint)
	{
#if ENABLE_PIXEL_STREAMING_2
		if (FModuleManager::Get().IsModuleLoaded("PixelStreaming2"))
		{
			Packet->Write(TEXT("PSVersion"), TEXT("PS2"));
#if WITH_EDITOR
			if (FModuleManager::Get().IsModuleLoaded("PixelStreaming2Editor"))
			{
				FString ViewerPort = FString::FromInt(IPixelStreaming2EditorModule::Get().GetViewerPort());
				if (ViewerPort.IsEmpty() || !ViewerPort.IsNumeric())
				{
					ViewerPort = TEXT("80");
				}
				Packet->Write(TEXT("PSHostPort"), ViewerPort);
			}
#endif //WITH_EDITOR
		}
#else //ENABLE_PIXEL_STREAMING_2
		if (FModuleManager::Get().IsModuleLoaded("PixelStreaming"))
		{
			Packet->Write(TEXT("PSVersion"), TEXT("PS1"));
#if WITH_EDITOR
			if (FModuleManager::Get().IsModuleLoaded("PixelStreamingEditor"))
			{
				FString ViewerPort;
				ViewerPort = FString::FromInt(IPixelStreamingEditorModule::Get().GetViewerPort());
				if (ViewerPort.IsEmpty() || !ViewerPort.IsNumeric())
				{
					ViewerPort = TEXT("80");
				}
				Packet->Write(TEXT("PSHostPort"), ViewerPort);
			}
#endif //WITH_EDITOR
		}
#endif //ENABLE_PIXEL_STREAMING_2
	}
}

void FRemoteSessionHost::OnReceiveHello(IBackChannelPacket& Message)
{
	// Need to read the message values one by one to get the correct data we need
	Message.ResetReading();

	FString VerString;
	Message.Read(TEXT("Version"), VerString);

	RemotePixelStreamingVersion.Empty();
	Message.Read(TEXT("PSVersion"), RemotePixelStreamingVersion);
	if (RemotePixelStreamingVersion.IsEmpty())
	{
		bIsUsingPixelStreaming = false;
	}
	else
	{
		bIsUsingPixelStreaming = true;
	}
}

void FRemoteSessionHost::Tick(float DeltaTime)
{
	// non-threaded listener
	if (IsConnected() == false)
	{
		if (Listener.IsValid() && IsListenerConnected)
		{
			Listener->Close();
			Listener = nullptr;

			//reset the host TCP socket
			StartListening(HostTCPPort);
			IsListenerConnected = false;
		}
        
        if (Listener.IsValid())
        {
            Listener->WaitForConnection(0, [this](TSharedRef<IBackChannelSocketConnection> InConnection) {
                CloseConnections();
				Connection = InConnection;
                CreateOSCConnection(InConnection);
                return true;
            });
        }
	}
	
	FRemoteSessionRole::Tick(DeltaTime);
}
