// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RemoteSessionRole.h"

struct FRemoteSessionChannelInfo;

class IBackChannelSocketConnection;
class FRecordingMessageHandler;
class FFrameGrabber;
class IImageWrapper;
class FRemoteSessionInputChannel;


class FRemoteSessionHost : public FRemoteSessionRole, public TSharedFromThis<FRemoteSessionHost>
{
public:

	FRemoteSessionHost(TArray<FRemoteSessionChannelInfo> SupportedChannels);
	~FRemoteSessionHost();	

	bool StartListening(const uint16 Port);

	void SetScreenSharing(const bool bEnabled);

	virtual void Tick(float DeltaTime) override;

protected:

	/* Closes all connections. Called by public Close() function which first send a graceful goodbye */
	virtual void	CloseConnections() override;

	virtual bool 	ProcessStateChange(const ConnectionState NewState, const ConnectionState OldState) override;

	virtual void 	BindEndpoints(TBackChannelSharedPtr<IBackChannelConnection> InConnection) override;

	void			SendChannelListToConnection();
	
	bool			ProcessIncomingConnection(TSharedRef<IBackChannelSocketConnection> NewConnection);

	void			OnReceiveHello(IBackChannelPacket& Message);
	virtual void	AppendExtraInfoToBackChannelPacket(TBackChannelSharedPtr<IBackChannelPacket> Packet) override;

	TSharedPtr<IBackChannelSocketConnection> Listener;

	/** Saved information about the editor and viewport we possessed, so we can restore it after exiting VR mode */
	float SavedEditorDragTriggerDistance;

	/** Host's TCP port */
	uint16 HostTCPPort;

	/** True if the host TCP socket is connected*/
	bool IsListenerConnected;
};
