// Copyright Epic Games, Inc. All Rights Reserved.

#include "RemoteSessionModule.h"
#include "Channels/RemoteSessionChannel.h"
#include "RemoteSessionHost.h"
#include "Misc/CommandLine.h"
#include "Misc/CoreDelegates.h"
#include "Modules/ModuleManager.h"
#include "RemoteSessionClient.h"
#include "Misc/App.h"

#include "EngineAnalytics.h"
#include "AnalyticsEventAttribute.h"
#include "Interfaces/IAnalyticsProvider.h"
#include "IAnalyticsProviderET.h"

#if WITH_NETWORK_SERVICE_DISCOVERY
#include "INetworkServiceDiscovery.h"
#endif

#if WITH_EDITOR
	#include "Editor.h"
	#include "ISettingsModule.h"
	#include "ISettingsSection.h"
#else
	#include "HAL/IConsoleManager.h"
#endif


#define LOCTEXT_NAMESPACE "FRemoteSessionModule"

#define REMOTE_SESSION_VERSION_STRING TEXT("1.1")
#define REMOTE_SESSION_LEGACY_VERSION_STRING TEXT("1.0.5")


FString IRemoteSessionModule::GetLocalVersion()
{
	return REMOTE_SESSION_VERSION_STRING;
}

FString IRemoteSessionModule::GetLastSupportedVersion()
{
	return REMOTE_SESSION_LEGACY_VERSION_STRING;
}

namespace UE::RemoteSession::Private
{
	static FString GClientBuildInfo;
	static TSharedPtr<IAnalyticsProviderET> GAnalyticsProvider;
}

void IRemoteSessionModule::SetClientBuildInfo(const FString& BuildInfo)
{
	UE::RemoteSession::Private::GClientBuildInfo = BuildInfo;
}

FString IRemoteSessionModule::GetClientBuildInfo()
{
	return UE::RemoteSession::Private::GClientBuildInfo;
}

void IRemoteSessionModule::SetAnalyticsProvider(TSharedPtr<IAnalyticsProviderET> Provider)
{
	UE::RemoteSession::Private::GAnalyticsProvider = MoveTemp(Provider);
}

void FRemoteSessionModule::SetAutoStartWithPIE(bool bEnable)
{
	bAutoHostWithPIE = bEnable;
}

void FRemoteSessionModule::StartupModule()
{	
	if (PLATFORM_DESKTOP 
		&& IsRunningDedicatedServer() == false 
		&& IsRunningCommandlet() == false)
	{
#if WITH_EDITOR
		PostPieDelegate = FEditorDelegates::PostPIEStarted.AddRaw(this, &FRemoteSessionModule::OnPIEStarted);
		EndPieDelegate = FEditorDelegates::EndPIE.AddRaw(this, &FRemoteSessionModule::OnPIEEnded);
#endif
		GameStartDelegate = FCoreDelegates::OnFEngineLoopInitComplete.AddRaw(this, &FRemoteSessionModule::OnPostInit);
		GameStartDelegate = FCoreDelegates::OnEnginePreExit.AddRaw(this, &FRemoteSessionModule::OnPreExit);

	}
}

void FRemoteSessionModule::ShutdownModule()
{
	// This function may be called during shutdown to clean up your module.  For modules that support dynamic reloading,
	// we call this function before unloading the module.
#if WITH_EDITOR
	if (PostPieDelegate.IsValid())
	{
		FEditorDelegates::PostPIEStarted.Remove(PostPieDelegate);
	}

	if (EndPieDelegate.IsValid())
	{
		FEditorDelegates::EndPIE.Remove(EndPieDelegate);
	}

	// unregister settings
	ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

	if (SettingsModule != nullptr)
	{
		SettingsModule->UnregisterSettings("Project", "Plugins", "RemoteSession");
	}
#endif

	if (GameStartDelegate.IsValid())
	{
		FCoreDelegates::OnFEngineLoopInitComplete.Remove(GameStartDelegate);
	}
}

void FRemoteSessionModule::OnPostInit()
{
#if WITH_EDITOR
	// register settings
	ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

	if (SettingsModule != nullptr)
	{
		ISettingsSectionPtr SettingsSection = SettingsModule->RegisterSettings("Project", "Plugins", "RemoteSession",
			LOCTEXT("RemoteSessionSettingsName", "Remote Session"),
			LOCTEXT("RemoteSessionSettingsDescription", "Configure the Remote Session plugin."),
			GetMutableDefault<URemoteSessionSettings>()
		);

		if (SettingsSection.IsValid())
		{
			SettingsSection->OnModified().BindRaw(this, &FRemoteSessionModule::HandleSettingsSaved);
		}
	}
#endif // WITH_EDITOR

	// Call this to trigger set up based on current settings
	HandleSettingsSaved();

	bool IsHostGame = PLATFORM_DESKTOP
		&& GIsEditor == false
		&& IsRunningDedicatedServer() == false
		&& IsRunningCommandlet() == false;

	if (IsHostGame && bAutoHostWithGame)
	{
		InitHost();
	}
}


void FRemoteSessionModule::OnPreExit()
{
	if (IsHostConnected())
	{
		StopHost(TEXT("Host Exited"));
	}

	if (Client.IsValid())
	{
		StopClient(Client, TEXT("Client Exited"));
	}
}

/** Callback for when the settings were saved. */
bool FRemoteSessionModule::HandleSettingsSaved()
{
	URemoteSessionSettings* Settings = URemoteSessionSettings::StaticClass()->GetDefaultObject<URemoteSessionSettings>();

	bAutoHostWithPIE = Settings->bAutoHostWithPIE;
	bAutoHostWithGame = Settings->bAutoHostWithPIE;
	DefaultPort = Settings->HostPort;

	// port can be overriden on the command line
	FParse::Value(FCommandLine::Get(), TEXT("remote.port="), DefaultPort);

	TArray<FRemoteSessionChannelInfo> KnownChannels = FRemoteSessionChannelRegistry::Get().GetRegisteredFactories();

	// check channels
	for (const auto& Channel : Settings->AllowedChannels)
	{
		if (KnownChannels.ContainsByPredicate([Channel](const FRemoteSessionChannelInfo& Info) {
			return Info.Type == Channel;
			}))
		{
			UE_LOGF(LogRemoteSession, Error, "Channel %ls in the ini file allow list is not a recognized channel.", *Channel);
		}
	}

	for (const auto& Channel : Settings->DeniedChannels)
	{
		if (KnownChannels.ContainsByPredicate([Channel](const FRemoteSessionChannelInfo& Info) {
			return Info.Type == Channel;
			}))
		{
			UE_LOGF(LogRemoteSession, Error, "Channel %ls in the ini file deny list is not a recognized channel.", *Channel);
		}
	}

	return true;
}


void FRemoteSessionModule::AddChannelFactory(const FStringView InChannelName, ERemoteSessionChannelMode InHostMode, TWeakPtr<IRemoteSessionChannelFactoryWorker> Worker)
{
	FRemoteSessionChannelRegistry::Get().RegisterChannelFactory(*FString::ConstructFromPtrSize(InChannelName.GetData(), InChannelName.Len()), InHostMode, Worker);
}

void FRemoteSessionModule::RemoveChannelFactory(TWeakPtr<IRemoteSessionChannelFactoryWorker> Worker)
{
	FRemoteSessionChannelRegistry::Get().RemoveChannelFactory(Worker);
}


void FRemoteSessionModule::OnPIEStarted(bool bSimulating)
{
	if (bAutoHostWithPIE)
	{
		InitHost();
	}
}

void FRemoteSessionModule::OnPIEEnded(bool bSimulating)
{
	// always stop, in case it was started via the console
	StopHost(TEXT("PIE Ended"));
}

TSharedPtr<IRemoteSessionRole> FRemoteSessionModule::CreateClient(const TCHAR* RemoteAddress)
{
	// todo - remove this and allow multiple clients (and hosts) to be created
	if (Client.IsValid())
	{
		StopClient(Client, TEXT("New Client"));
	}
	Client = MakeShareable(new FRemoteSessionClient(RemoteAddress));
	return Client;
}

void FRemoteSessionModule::StopClient(TSharedPtr<IRemoteSessionRole> InClient, const FString& InReason)
{
	if (InClient.IsValid())
	{
		TSharedPtr<FRemoteSessionClient> CastClient = StaticCastSharedPtr<FRemoteSessionClient>(InClient);
		CastClient->Close(InReason);
			
		if (CastClient == Client)
		{
			Client = nullptr;
		}
	}
}

void FRemoteSessionModule::InitHost(const int16 Port /*= 0*/)
{
	if (Host.IsValid())
	{
		// Route through StopHost so the previous host's delegate, mDNS advertisement,
		// and connection are torn down correctly. Preserve HostAnalyticsEventName /
		// HostAnalyticsEventMode across this implicit teardown: a caller may have
		// just set them expecting the values to apply to the new host being
		// initialised here.
		const FString PreservedEventName = HostAnalyticsEventName;
		const EAnalyticsRecordEventMode PreservedEventMode = HostAnalyticsEventMode;
		StopHost(TEXT("InitHost replacing existing host"));
		HostAnalyticsEventName = PreservedEventName;
		HostAnalyticsEventMode = PreservedEventMode;
	}

	TArray<FRemoteSessionChannelInfo> SupportedChannels;

	const URemoteSessionSettings* Settings = URemoteSessionSettings::StaticClass()->GetDefaultObject<URemoteSessionSettings>();

	SupportedChannels = FRemoteSessionChannelRegistry::Get().GetRegisteredFactories();

	if (Settings->AllowedChannels.Num())
	{
		// Remove anything not in the allow list
		SupportedChannels = SupportedChannels.FilterByPredicate([Settings](const FRemoteSessionChannelInfo& Info) {
			return Settings->AllowedChannels.Contains(Info.Type);
		});
	}


	if (Settings->DeniedChannels.Num())
	{
		// Remove anything in the denied list
		SupportedChannels = SupportedChannels.FilterByPredicate([Settings](const FRemoteSessionChannelInfo& Info) {
			return Settings->DeniedChannels.Contains(Info.Type) == false;
		});
	}

	int16 SelectedPort = Port ? Port : (int16)DefaultPort;
	if (TSharedPtr<FRemoteSessionHost> NewHost = CreateHostInternal(MoveTemp(SupportedChannels), SelectedPort))
	{
		Host = NewHost;
		ConnectionChangeDelegateHandle = Host->RegisterConnectionChangeDelegate(
			FOnRemoteSessionConnectionChange::FDelegate::CreateRaw(this, &FRemoteSessionModule::OnHostConnectionChanged));
		UE_LOGF(LogRemoteSession, Log, "Started listening on port %d", SelectedPort);

#if WITH_NETWORK_SERVICE_DISCOVERY
		// Advertise via mDNS so mobile clients can discover this host
		if (INetworkServiceDiscoveryModule* Discovery = FModuleManager::GetModulePtr<INetworkServiceDiscoveryModule>("NetworkServiceDiscovery"))
		{
			TMap<FString, FString> TxtRecord;
			TxtRecord.Add(TEXT("version"), IRemoteSessionModule::GetLocalVersion());
			TxtRecord.Add(TEXT("project"), FApp::GetProjectName());
			FString ServiceName = FString::Printf(TEXT("%s on %s"), FApp::GetProjectName(), FPlatformProcess::ComputerName());
			Discovery->RegisterService(ServiceName, TEXT("_unrealremote._tcp."), SelectedPort, TxtRecord);
		}
#endif
	}
	else
	{
		UE_LOGF(LogRemoteSession, Error, "Failed to start host listening on port %d", SelectedPort);
	}
}

void FRemoteSessionModule::StopHost(const FString& InReason)
{ 
#if WITH_NETWORK_SERVICE_DISCOVERY
	// Stop mDNS advertisement — use the same service name as registration
	if (INetworkServiceDiscoveryModule* Discovery = FModuleManager::GetModulePtr<INetworkServiceDiscoveryModule>("NetworkServiceDiscovery"))
	{
		FString ServiceName = FString::Printf(TEXT("%s on %s"), FApp::GetProjectName(), FPlatformProcess::ComputerName());
		Discovery->UnregisterService(ServiceName);
	}
#endif

	if (Host.IsValid())
	{
		Host->RemoveAllDelegates(this);
	}
	ConnectionChangeDelegateHandle.Reset();
	HostAnalyticsEventName = DefaultHostAnalyticsEventName;
	HostAnalyticsEventMode = DefaultHostAnalyticsEventMode;

	if (IsHostConnected())
	{
		Host->Close(InReason);
	}
	Host = nullptr;
}

void FRemoteSessionModule::OnHostConnectionChanged(IRemoteSessionRole* Role, ERemoteSessionConnectionChange Change)
{
	if (Change != ERemoteSessionConnectionChange::Connected || Role == nullptr)
	{
		return;
	}

	IAnalyticsProviderET* Provider = UE::RemoteSession::Private::GAnalyticsProvider.Get();
	if (Provider == nullptr && FEngineAnalytics::IsAvailable())
	{
		Provider = &FEngineAnalytics::GetProvider();
	}
	if (Provider == nullptr)
	{
		return;
	}

	TArray<FAnalyticsEventAttribute> Attrs;
	auto OrNone = [](const FString& Value) { return Value.IsEmpty() ? FString(TEXT("none")) : Value; };

	Attrs.Emplace(TEXT("Role"), TEXT("Host"));
	Attrs.Emplace(TEXT("LocalVersion"), IRemoteSessionModule::GetLocalVersion());
	Attrs.Emplace(TEXT("RemoteVersion"), OrNone(Role->GetRemoteVersion()));
	Attrs.Emplace(TEXT("UsingPixelStreaming"), Role->IsUsingPixelStreaming());
	Attrs.Emplace(TEXT("RemotePixelStreamingVersion"), OrNone(Role->GetRemotePixelStreamingVersion()));
	Attrs.Emplace(TEXT("RemoteBuildInfo"), OrNone(Role->GetRemoteBuildInfo()));

	Provider->RecordEvent(FString(HostAnalyticsEventName), Attrs, HostAnalyticsEventMode);
}

bool FRemoteSessionModule::IsHostConnected() const
{
	return Host.IsValid() && Host->IsConnected();
}

TSharedPtr<IRemoteSessionRole> FRemoteSessionModule::GetHost() const
{
	return Host;
}

TSharedPtr<IRemoteSessionUnmanagedRole> FRemoteSessionModule::CreateHost(TArray<FRemoteSessionChannelInfo> SupportedChannels, int32 Port) const
{
	return CreateHostInternal(MoveTemp(SupportedChannels), Port);
}

TSharedPtr<FRemoteSessionHost> FRemoteSessionModule::CreateHostInternal(TArray<FRemoteSessionChannelInfo> SupportedChannels, int32 Port) const
{
#if UE_BUILD_SHIPPING
	const URemoteSessionSettings* Settings = URemoteSessionSettings::StaticClass()->GetDefaultObject<URemoteSessionSettings>();
	if (Settings->bAllowInShipping == false)
	{
		UE_LOGF(LogRemoteSession, Log, "RemoteSession is disabled. Shipping=1");
		return TSharedPtr<FRemoteSessionHost>();
	}
#endif

	TSharedPtr<FRemoteSessionHost> NewHost = MakeShared<FRemoteSessionHost>(MoveTemp(SupportedChannels));
	if (NewHost->StartListening(Port))
	{
		return NewHost;
	}
	return TSharedPtr<FRemoteSessionHost>();
}

void FRemoteSessionModule::Tick(float DeltaTime)
{
	if (Client.IsValid())
	{
		Client->Tick(DeltaTime);
	}

	if (Host.IsValid())
	{
		Host->Tick(DeltaTime);
	}
}

IMPLEMENT_MODULE(FRemoteSessionModule, RemoteSession)

FAutoConsoleCommand GRemoteHostCommand(
	TEXT("remote.host"),
	TEXT("Starts a remote viewer host"),
	FConsoleCommandDelegate::CreateStatic(
		[]()
	{
		if (FRemoteSessionModule* Viewer = FModuleManager::LoadModulePtr<FRemoteSessionModule>("RemoteSession"))
		{
			Viewer->InitHost();
		}
	})
);

FAutoConsoleCommand GRemoteDisconnectCommand(
	TEXT("remote.disconnect"),
	TEXT("Disconnect remote viewer"),
	FConsoleCommandDelegate::CreateStatic(
		[]()
	{
		if (FRemoteSessionModule* Viewer = FModuleManager::LoadModulePtr<FRemoteSessionModule>("RemoteSession"))
		{
			//Viewer->StopClient();
			Viewer->StopHost(TEXT("Console Disconnect"));
		}
	})
);

FAutoConsoleCommand GRemoteAutoPIECommand(
	TEXT("remote.autopie"),
	TEXT("enables remote with pie"),
	FConsoleCommandDelegate::CreateStatic(
		[]()
	{
		if (FRemoteSessionModule* Viewer = FModuleManager::LoadModulePtr<FRemoteSessionModule>("RemoteSession"))
		{
			Viewer->SetAutoStartWithPIE(true);
		}
	})
);

#undef LOCTEXT_NAMESPACE
