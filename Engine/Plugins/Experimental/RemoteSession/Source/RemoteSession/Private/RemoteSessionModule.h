// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RemoteSession.h"
#include "Interfaces/IAnalyticsProvider.h"
#include "Stats/Stats.h"
#include "Tickable.h"

enum class EStatFlags : uint8;
struct FStatGroup_STATGROUP_Tickables;
struct TStatIdData;

enum class ERemoteSessionConnectionChange : int32;

class FRemoteSessionHost;
class FRemoteSessionClient;
class IRemoteSessionChannelFactoryWorker;



class FRemoteSessionModule : public IRemoteSessionModule, public FTickableGameObject
{
private:

	TSharedPtr<FRemoteSessionHost>							Host;
	TSharedPtr<FRemoteSessionClient>						Client;

	int32													DefaultPort = IRemoteSessionModule::kDefaultPort;

	bool													bAutoHostWithPIE = true;
	bool													bAutoHostWithGame = true;

	FDelegateHandle PostPieDelegate;
	FDelegateHandle EndPieDelegate;
	FDelegateHandle GameStartDelegate;
	FDelegateHandle ConnectionChangeDelegateHandle;

	static constexpr const TCHAR* DefaultHostAnalyticsEventName = TEXT("Usage.RemoteSession.Connected");
	static constexpr EAnalyticsRecordEventMode DefaultHostAnalyticsEventMode = EAnalyticsRecordEventMode::Cached;
	FString HostAnalyticsEventName = DefaultHostAnalyticsEventName;
	EAnalyticsRecordEventMode HostAnalyticsEventMode = DefaultHostAnalyticsEventMode;

public:

	//~ Begin IRemoteSessionModule interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	void AddChannelFactory(const FStringView InChannelName, ERemoteSessionChannelMode InHostMode, TWeakPtr<IRemoteSessionChannelFactoryWorker> Worker) override;
	void RemoveChannelFactory(TWeakPtr<IRemoteSessionChannelFactoryWorker> Worker) override;

	virtual TSharedPtr<IRemoteSessionRole>	CreateClient(const TCHAR* RemoteAddress) override;
	virtual void StopClient(TSharedPtr<IRemoteSessionRole> InClient, const FString& InReason) override;

	virtual void InitHost(const int16 Port = 0) override;
	virtual bool IsHostRunning() const override { return Host.IsValid(); }
	virtual bool IsHostConnected() const override;
	virtual void StopHost(const FString& InReason) override;
	virtual TSharedPtr<IRemoteSessionRole> GetHost() const override;
	virtual TSharedPtr<IRemoteSessionUnmanagedRole> CreateHost(TArray<FRemoteSessionChannelInfo> SupportedChannels, int32 Port) const override;
	virtual void SetHostAnalyticsEventName(const FString& EventName) override { HostAnalyticsEventName = EventName; }
	virtual void SetHostAnalyticsEventMode(EAnalyticsRecordEventMode Mode) override { HostAnalyticsEventMode = Mode; }
	//~ End IRemoteSessionModule interface

	//~ Begin FTickableGameObject interface
	virtual TStatId GetStatId() const override { RETURN_QUICK_DECLARE_CYCLE_STAT(FRemoteSession, STATGROUP_Tickables); }
	virtual bool IsTickable() const override { return true; }
	virtual void Tick(float DeltaTime) override;
	//~ End FTickableGameObject interface

	void SetAutoStartWithPIE(bool bEnable);

private:
	void OnPostInit();
	void OnPreExit();
	bool HandleSettingsSaved();

	TSharedPtr<FRemoteSessionHost> CreateHostInternal(TArray<FRemoteSessionChannelInfo> SupportedChannels, int32 Port) const;

	void OnPIEStarted(bool bSimulating);
	void OnPIEEnded(bool bSimulating);

	void OnHostConnectionChanged(IRemoteSessionRole* Role, ERemoteSessionConnectionChange Change);
};
