// Copyright Epic Games, Inc. All Rights Reserved.


#include "RemoteSessionRole.h"
#include "BackChannel/Protocol/OSC/BackChannelOSCConnection.h"
#include "RemoteSession.h"
#include "BackChannel/Protocol/OSC/BackChannelOSCMessage.h"

#include "Async/Async.h"
#include "BackChannel/Transport/IBackChannelSocketConnection.h"
#include "Channels/RemoteSessionChannel.h"
#include "Modules/ModuleManager.h"


DEFINE_LOG_CATEGORY(LogRemoteSession);

const TCHAR* LexToString(ERemoteSessionChannelMode InMode)
{
	switch (InMode)
	{
	case ERemoteSessionChannelMode::Unknown:
		return TEXT("Unknown");
	case ERemoteSessionChannelMode::Read:
		return TEXT("Read");
	case ERemoteSessionChannelMode::Write:
		return TEXT("Write");
	default:
		check(false);
		return TEXT("Unknown");
	}
}

void LexFromString(ERemoteSessionChannelMode& Value, const TCHAR* String)
{
	Value = ERemoteSessionChannelMode::Unknown;

	for (int i = 0; i < (int)ERemoteSessionChannelMode::MaxValue; ++i)
	{
		if (FCString::Stricmp(LexToString((ERemoteSessionChannelMode)i), String) == 0)
		{
			Value = (ERemoteSessionChannelMode)i;
			return;
		}
	}
}

// Helpers for the /RS.V2Hello endpoint. OSC's Read is positional (the InName argument
// is ignored), so we encode named key/value pairs as consecutive OSC string slots:
// key, value, key, value, ... — the reader walks pairs and builds a map. Adding,
// removing, or reordering pairs is safe across peer versions.
namespace
{
	void WriteNamedPair(IBackChannelPacket& Packet, const TCHAR* Key, const FString& Value)
	{
		Packet.Write(TEXT("Key"), FString(Key));
		Packet.Write(TEXT("Value"), Value);
	}

	TMap<FString, FString> ReadNamedPairs(IBackChannelPacket& Message)
	{
		TMap<FString, FString> Result;
		Message.ResetReading();

		// Bound the loop by the actual OSC argument count so we don't trigger the
		// "no more tags to read" warning on what would otherwise be the loop's
		// natural terminator. Same cast pattern as OnReceiveChannelList.
		FBackChannelOSCMessage& OSCMessage = static_cast<FBackChannelOSCMessage&>(Message);
		const int32 NumPairs = OSCMessage.GetArgumentCount() / 2;

		for (int32 i = 0; i < NumPairs; ++i)
		{
			FString Key;
			if (Message.Read(TEXT("Key"), Key) != 0)
			{
				break;
			}
			FString Value;
			if (Message.Read(TEXT("Value"), Value) != 0)
			{
				break;
			}
			Result.Add(MoveTemp(Key), MoveTemp(Value));
		}
		return Result;
	}
}

// Simple struct use to establish a cancellation token for async tasks.
//
struct FRemoteSessionRoleCancellationToken
{
};

FRemoteSessionRole::FRemoteSessionRole()
	:bIsUsingPixelStreaming(false)
{
	ExternalHelloRouteDelegateHandles.Empty();

}

FRemoteSessionRole::~FRemoteSessionRole()
{
	RemoveRouteDelegates();
	CloseConnections();
}

void FRemoteSessionRole::CloseConnections()
{
	// order is specific since OSC uses the connection, and
	// dispatches to channels
	ConnectionChangeDelegate.Broadcast(this, ERemoteSessionConnectionChange::Disconnected);

	OSCConnection = nullptr;
	Connection = nullptr;
	ClearChannels();
	CurrentState = ConnectionState::Disconnected;
	PendingState = ConnectionState::Unknown;
	RemoteVersion = TEXT("");
	RemoteBuildInfo.Reset();
	LastPingTime = 0;
	SecondsForPeerResponse = 0;
	LastReponseTime = 0;
	FBackChannelOSCMessage::SetLegacyMode(false);
}

void FRemoteSessionRole::Close(const FString& Message)
{
	FScopeLock Tmp(&CriticalSectionForMainThread);
	ErrorMessage = Message;

	if (OSCConnection && OSCConnection->IsConnected())
	{
		SendGoodbye(Message);
	}

	CloseConnections();
}

void FRemoteSessionRole::CloseWithError(const FString& Message)
{
	Close(Message);
}

void FRemoteSessionRole::Tick(float DeltaTime)
{
	const URemoteSessionSettings* ConnectionSettings = GetDefault<URemoteSessionSettings>();

	const float kTimeout = FPlatformMisc::IsDebuggerPresent() ? ConnectionSettings->ConnectionTimeoutWhenDebugging : ConnectionSettings->ConnectionTimeout;
	const float kPingTime = ConnectionSettings->PingTime;

	if (PendingState != ConnectionState::Unknown)
	{
		UE_LOGF(LogRemoteSession, Log, "Processing change from %ls to %ls", LexToString(CurrentState), LexToString(PendingState));
		// pass this in so code can know where we came from if needed
		ConnectionState OldState = CurrentState;
		CurrentState = PendingState;
		PendingState = ConnectionState::Unknown;
		if (ProcessStateChange(CurrentState, OldState))
		{
			UE_LOGF(LogRemoteSession, Log, "Changed state to %ls", LexToString(CurrentState));

			if (CurrentState == ConnectionState::Connected)
			{
				UE_LOGF(LogRemoteSession, Log, "Starting OSC receive thread for future messages");
				OSCConnection->StartReceiveThread();
				ConnectionChangeDelegate.Broadcast(this, ERemoteSessionConnectionChange::Connected);
			}
		}
		else
		{
			CloseWithError(TEXT("State change failed! Closing connection"));
		}
	}

	bool DidHaveConnection = OSCConnection.IsValid() && !OSCConnection->IsConnected();

	bool HaveLowLevelConnection = OSCConnection.IsValid() && OSCConnection->IsConnected();

	if (HaveLowLevelConnection)
	{
		if (GetCurrentState() == ConnectionState::Disconnected)
		{
			SetPendingState(ConnectionState::UnversionedConnection);
		}
		else
		{
			if (OSCConnection->IsThreaded() == false)
			{
				OSCConnection->ReceiveAndDispatchMessages();
			}

			if (GetCurrentState() == ConnectionState::Connected)
			{
				for (auto& Channel : Channels)
				{
					Channel->Tick(DeltaTime);
				}
			}
		}

		// Check if we're in an error state
		IBackChannelSocketConnection::FConnectionStats Stats = Connection->GetConnectionStats();

		// todo - needed for legacy?
		const double TimeNow = FPlatformTime::Seconds();
		
		// Send a ping periodically. This is required to keep things alive
		if (TimeNow - LastPingTime >= kPingTime)
		{
			SendPing();
		}

		if (LastReponseTime == 0)
		{
			LastReponseTime = TimeNow;
		}
		else
		{
			LastReponseTime = FMath::Max(LastReponseTime, Stats.LastReceiveTime);
		}

		//const double TimeWithErrors = Stats.LastErrorTime - Stats.Last;
		const double TimeWithNoReceive = TimeNow - LastReponseTime;

		if (TimeWithNoReceive >= kTimeout)
		{
			CloseWithError(FString::Printf(TEXT("Closing connection to %s after receiving no data for %.02f seconds"), *Connection->GetDescription(), TimeWithNoReceive));
		}
	}
	else if (DidHaveConnection)
	{
		UE_LOGF(LogRemoteSession, Warning, "Connection %ls has disconnected.", *OSCConnection->GetDescription());
		OSCConnection = nullptr;
	}
}

bool FRemoteSessionRole::IsConnected() const
{
	// just check this is valid, when it's actually disconnected we do some error
	// handling and clean this up
	return OSCConnection.IsValid();
}

bool FRemoteSessionRole::HasError() const
{
	FScopeLock Tmp(&CriticalSectionForMainThread);
	return ErrorMessage.Len() > 0;
}

bool FRemoteSessionRole::IsLegacyConnection() const
{
	return RemoteVersion == IRemoteSessionModule::GetLastSupportedVersion();
}


FString FRemoteSessionRole::GetErrorMessage() const
{
	FScopeLock Tmp(&CriticalSectionForMainThread);
	return ErrorMessage;
}


void FRemoteSessionRole::CreateOSCConnection(TSharedRef<IBackChannelSocketConnection> InConnection)
{
	OSCConnection = MakeShareable(new FBackChannelOSCConnection(InConnection));

	SetPendingState(ConnectionState::UnversionedConnection);
}

/* Registers a delegate for notifications of connection changes*/
FDelegateHandle FRemoteSessionRole::RegisterConnectionChangeDelegate(FOnRemoteSessionConnectionChange::FDelegate InDelegate)
{
	return ConnectionChangeDelegate.Add(InDelegate);
}


FDelegateHandle FRemoteSessionRole::RegisterChannelChangeDelegate(FOnRemoteSessionChannelChange::FDelegate InDelegate)
{
	return ChannelChangeDelegate.Add(InDelegate);
}

/* Register for notifications when the host sends a list of available channels */
FDelegateHandle FRemoteSessionRole::RegisterChannelListDelegate(FOnRemoteSessionReceiveChannelList::FDelegate InDelegate)
{
	return ReceiveChannelListDelegate.Add(InDelegate);
}

void FRemoteSessionRole::RegisterHelloRouteMessageDelegate(FBackChannelRouteDelegate::FDelegate InDelegate)
{
	// Keep a persistent copy so the delegate survives reconnects
	ExternalHelloRouteDelegates.Add(InDelegate);

	if (BackChannelConnection.IsValid())
	{
		TBackChannelSharedPtr<IBackChannelConnection> CurrentConnection = BackChannelConnection.Pin();
		FDelegateHandle Handle = CurrentConnection->AddRouteDelegate(kHelloEndPoint, InDelegate);
		ExternalHelloRouteDelegateHandles.Add(Handle);
	}
}

void FRemoteSessionRole::RemoveAllDelegates(FDelegateUserObject UserObject)
{
	ConnectionChangeDelegate.RemoveAll(UserObject);
	ChannelChangeDelegate.RemoveAll(UserObject);
	ReceiveChannelListDelegate.RemoveAll(UserObject);

	// Purge persistent delegates bound to this object so they aren't re-registered on reconnect
	ExternalHelloRouteDelegates.RemoveAll([UserObject](const FBackChannelRouteDelegate::FDelegate& Delegate)
	{
		return Delegate.IsBoundToObject(UserObject);
	});

	if (!BackChannelConnection.IsValid())
	{
		return;
	}

	TBackChannelSharedPtr<IBackChannelConnection> CurrentConnection = BackChannelConnection.Pin();
	for (FDelegateHandle& Handle : ExternalHelloRouteDelegateHandles)
	{
		CurrentConnection->RemoveRouteDelegate(kHelloEndPoint, Handle);
	}

	ExternalHelloRouteDelegateHandles.Empty();
}

void FRemoteSessionRole::RemoveRouteDelegates()
{
	if (!BackChannelConnection.IsValid())
	{
		return;
	}
	TBackChannelSharedPtr<IBackChannelConnection> InConnection = BackChannelConnection.Pin();

	InConnection->RemoveRouteDelegate(kHelloEndPoint, RouteDelegates.Hello);
	InConnection->RemoveRouteDelegate(kHelloV2EndPoint, RouteDelegates.HelloV2);
	InConnection->RemoveRouteDelegate(kGoodbyeEndPoint, RouteDelegates.Goodbye);
	InConnection->RemoveRouteDelegate(kPingEndPoint, RouteDelegates.Ping);
	InConnection->RemoveRouteDelegate(kPongEndPoint, RouteDelegates.Pong);
	InConnection->RemoveRouteDelegate(kLegacyVersionEndPoint, RouteDelegates.LegacyReceive);
	InConnection->RemoveRouteDelegate(kChangeChannelEndPoint, RouteDelegates.ChangeChannel);

	for (FDelegateHandle& Handle : ExternalHelloRouteDelegateHandles)
	{
		InConnection->RemoveRouteDelegate(kHelloEndPoint, Handle);
	}
	ExternalHelloRouteDelegateHandles.Empty();

	BackChannelConnection = {};
}

void FRemoteSessionRole::BindEndpoints(TBackChannelSharedPtr<IBackChannelConnection> InConnection)
{
	RemoveRouteDelegates();

	auto Delegate = FBackChannelRouteDelegate::FDelegate::CreateRaw(this, &FRemoteSessionRole::OnReceiveHello);
	RouteDelegates.Hello = InConnection->AddRouteDelegate(kHelloEndPoint, Delegate);

	Delegate = FBackChannelRouteDelegate::FDelegate::CreateRaw(this, &FRemoteSessionRole::OnReceiveHelloV2);
	RouteDelegates.HelloV2 = InConnection->AddRouteDelegate(kHelloV2EndPoint, Delegate);

	Delegate = FBackChannelRouteDelegate::FDelegate::CreateRaw(this, &FRemoteSessionRole::OnReceiveGoodbye);
	RouteDelegates.Goodbye = InConnection->AddRouteDelegate(kGoodbyeEndPoint, Delegate);

	Delegate = FBackChannelRouteDelegate::FDelegate::CreateRaw(this, &FRemoteSessionRole::OnReceivePing);
	RouteDelegates.Ping = InConnection->AddRouteDelegate(kPingEndPoint, Delegate);

	Delegate = FBackChannelRouteDelegate::FDelegate::CreateRaw(this, &FRemoteSessionRole::OnReceivePong);
	RouteDelegates.Pong = InConnection->AddRouteDelegate(kPongEndPoint, Delegate);

	Delegate = FBackChannelRouteDelegate::FDelegate::CreateRaw(this, &FRemoteSessionRole::OnReceiveLegacyVersion);
	RouteDelegates.LegacyReceive = InConnection->AddRouteDelegate(kLegacyVersionEndPoint, Delegate);

	Delegate = FBackChannelRouteDelegate::FDelegate::CreateRaw(this, &FRemoteSessionRole::OnReceiveChangeChannel);
	RouteDelegates.ChangeChannel = InConnection->AddRouteDelegate(kChangeChannelEndPoint, Delegate);

	// Re-register all external Hello delegates (survives reconnects)
	ExternalHelloRouteDelegateHandles.Empty();
	for (FBackChannelRouteDelegate::FDelegate& ExternalDelegate : ExternalHelloRouteDelegates)
	{
		FDelegateHandle Handle = InConnection->AddRouteDelegate(kHelloEndPoint, ExternalDelegate);
		ExternalHelloRouteDelegateHandles.Add(Handle);
	}


	BackChannelConnection = InConnection;
}

void FRemoteSessionRole::SendLegacyVersionCheck()
{
	if (ensureMsgf(GetCurrentState() == ConnectionState::UnversionedConnection,
		TEXT("Can only send version check in an unversioned state. Current State is %s"),
		LexToString(GetCurrentState())))
	{
		TBackChannelSharedPtr<IBackChannelPacket> Packet = OSCConnection->CreatePacket();

		Packet->SetPath(kLegacyVersionEndPoint);
		Packet->Write(TEXT("Version"), IRemoteSessionModule::GetLastSupportedVersion());

		// now ask the client to start these channels
		//FBackChannelOSCMessage Msg(TEXT("/Version"));
		//Msg.Write(TEXT("Version"), FString(GetVersion()));

		OSCConnection->SendPacket(Packet);
	}
}

void FRemoteSessionRole::OnReceiveLegacyVersion(IBackChannelPacket& Message)
{
	if (IsStateCurrentOrPending(ConnectionState::Connected))
	{
		UE_LOGF(LogRemoteSession, Log, "Already connected with new protocol. Ignoring legacy connection message.");
		return;
	}

	FString VersionString;
	Message.Read(TEXT("Version"), VersionString);

	FString VersionErrorMessage;

	if (VersionString.Len() == 0)
	{
		VersionErrorMessage = TEXT("FRemoteSessionRole: Failed to read version string");
	}

	if (VersionString != IRemoteSessionModule::GetLocalVersion())
	{
		if (VersionString == IRemoteSessionModule::GetLastSupportedVersion())
		{
			UE_LOGF(LogRemoteSession, Warning, "Detected legacy version %ls. Setting compatibility options.", *VersionString);
			RemoteVersion = VersionString;
		}
		else
		{
			VersionErrorMessage = FString::Printf(TEXT("FRemoteSessionRole: Version mismatch. Local=%s, Remote=%s"), *IRemoteSessionModule::GetLocalVersion(), *VersionString);
		}
	}
	else
	{
		// this path should not be possible..
		bool value = ensureMsgf(false, TEXT("Received new protocol version through legacy handshake!"));
		if (value)
		{
			RemoteVersion = VersionString;
		}
	}

	if (VersionErrorMessage.Len() > 0)
	{
		UE_LOGF(LogRemoteSession, Error, "%ls", *VersionErrorMessage);
		UE_LOGF(LogRemoteSession, Log, "FRemoteSessionRole: Closing connection due to version mismatch");
		CloseWithError(VersionErrorMessage);
	}
	else
	{
		SetPendingState(ConnectionState::Connected);
		FBackChannelOSCMessage::SetLegacyMode(true);
	}
}

void FRemoteSessionRole::SendHello()
{
	if (ensureMsgf(GetCurrentState() == ConnectionState::UnversionedConnection,
		TEXT("Can only send version check in an unversioned state. Current State is %s"),
		LexToString(GetCurrentState())))
	{
		TBackChannelSharedPtr<IBackChannelPacket> Packet = OSCConnection->CreatePacket();

		// Send the v2 Hello FIRST. The state transition to Connected is driven by
		// receipt of the v1.1 Hello, and analytics fire on the next game-thread tick
		// after that. If v1.1 and v2 arrive in separate ReceiveAndDispatch passes
		// (likely on real networks with TCP segmentation), sending v2 first ensures
		// RemoteBuildInfo and any future v2 fields are populated before the analytics
		// broadcast — without coupling the state machine to the v2 endpoint.
		SendHelloV2();

		Packet->SetPath(kHelloEndPoint);
		Packet->Write(TEXT("Version"), IRemoteSessionModule::GetLocalVersion());

		AppendExtraInfoToBackChannelPacket(Packet);

		OSCConnection->SendPacket(Packet);
	}
}

void FRemoteSessionRole::SendHelloV2()
{
	if (!OSCConnection.IsValid())
	{
		return;
	}

	TBackChannelSharedPtr<IBackChannelPacket> Packet = OSCConnection->CreatePacket();
	Packet->SetPath(kHelloV2EndPoint);
	WriteNamedPair(*Packet, TEXT("BuildInfo"), IRemoteSessionModule::GetClientBuildInfo());
	// Future v2-only fields are added here as additional named pairs.
	OSCConnection->SendPacket(Packet);
}

void FRemoteSessionRole::OnReceiveHello(IBackChannelPacket& Message)
{
	if (IsStateCurrentOrPending(ConnectionState::Connected) || IsLegacyConnection())
	{
		UE_LOGF(LogRemoteSession, Log, "Received Hello to established legacy connection. Switching to new protocol");
		RemoteVersion = TEXT("");
		FBackChannelOSCMessage::SetLegacyMode(false);
	}

	Message.ResetReading();

	// Read the version
	FString VersionString;
	FString VersionErrorMessage;

	Message.Read(TEXT("Version"), VersionString);

	if (VersionString.Len() == 0)
	{
		VersionErrorMessage = TEXT("FRemoteSessionRole: Failed to read version string");
	}

	// No compatibility checks beyond this new version yet
	if (VersionString != IRemoteSessionModule::GetLocalVersion())
	{
		VersionErrorMessage = FString::Printf(TEXT("FRemoteSessionRole: Version mismatch. Local=%s, Remote=%s"), *IRemoteSessionModule::GetLocalVersion(), *VersionString);
	}
	else
	{
		RemoteVersion = VersionString;
	}

	if (VersionErrorMessage.Len() > 0)
	{
		UE_LOGF(LogRemoteSession, Error, "%ls", *VersionErrorMessage);
		UE_LOGF(LogRemoteSession, Log, "FRemoteSessionRole: Closing connection due to version mismatch");
		CloseWithError(VersionErrorMessage);
	}
	else
	{
		SetPendingState(ConnectionState::Connected);
	}
}

void FRemoteSessionRole::OnReceiveHelloV2(IBackChannelPacket& Message)
{
	const TMap<FString, FString> Pairs = ReadNamedPairs(Message);
	if (const FString* BuildInfo = Pairs.Find(TEXT("BuildInfo")))
	{
		RemoteBuildInfo = *BuildInfo;
	}
}

void FRemoteSessionRole::SendGoodbye(const FString& InReason)
{
	if (GetCurrentState() == ConnectionState::Connected && OSCConnection->IsConnected())
	{
		TBackChannelSharedPtr<IBackChannelPacket> Packet = OSCConnection->CreatePacket();

		Packet->SetPath(kGoodbyeEndPoint);
		Packet->Write(TEXT("Reason"), InReason);
		OSCConnection->SendPacket(Packet);
	}
}

void FRemoteSessionRole::OnReceiveGoodbye(IBackChannelPacket& Message)
{
	FString Reason;
	Message.Read(TEXT("Reason"), Reason);

	UE_LOGF(LogRemoteSession, Display, "FRemoteSessionRole: Closing due to goodbye from peer. '%ls'", *Reason);
	CancelCloseToken = MakeShared<FRemoteSessionRoleCancellationToken>();
	TWeakPtr<FRemoteSessionRoleCancellationToken> WeakCloseToken = CancelCloseToken;

	// Messages are received on a background thread and role operations should occur on
	// the main thread. Channels must be created on the main thread.
	AsyncTask(ENamedThreads::GameThread, [this, WeakCloseToken, Reason] {
		// We use a weak pointer to track the lifetime of this.  It is possible for this
		// to delete before the async class completes so we check the weak pointer to see
		// if it is valid. If it is not then we know the lifetime of this has expired and
		// we should not perform the block.
		//
		if (WeakCloseToken.IsValid())
		{
			ErrorMessage = Reason;
			CloseConnections();
			CancelCloseToken = {};
		}
	});
}

void FRemoteSessionRole::SendPing()
{
	if (OSCConnection.IsValid())
	{
		UE_LOGF(LogRemoteSession, Verbose, "Sending ping to %ls to check connection", *Connection->GetDescription());
		FBackChannelOSCMessage Msg(kPingEndPoint);
		OSCConnection->SendPacket(Msg);
		LastPingTime = FPlatformTime::Seconds();
	}
}

void FRemoteSessionRole::OnReceivePing(IBackChannelPacket& InMessage)
{
	// send pong
	if (OSCConnection.IsValid())
	{
		UE_LOGF(LogRemoteSession, Verbose, "Received Ping, Sending Pong");
		FBackChannelOSCMessage OutMessage(kPongEndPoint);
		OSCConnection->SendPacket(OutMessage);
	}
}

void FRemoteSessionRole::OnReceivePong(IBackChannelPacket& Message)
{
	SecondsForPeerResponse = FPlatformTime::Seconds() - LastPingTime;
	UE_LOGF(LogRemoteSession, Verbose, "Peer latency is currently %.02f seconds", SecondsForPeerResponse);
}

void FRemoteSessionRole::OnReceiveChannelChanged(IBackChannelPacket& Message)
{
	FString ChannelName;
	FString ChannelMode;

	Message.Read(TEXT("Name"), ChannelName);
	Message.Read(TEXT("Mode"), ChannelMode);

	AsyncTask(ENamedThreads::GameThread, [this, ChannelName, ChannelMode] {
		UE_LOGF(LogRemoteSession, Log, "Peer created Channel %ls with mode %ls", *ChannelName, *ChannelMode);

		TSharedPtr<IRemoteSessionChannel> NewChannel = GetChannel(*ChannelName);
		ChannelChangeDelegate.Broadcast(this, NewChannel, ERemoteSessionChannelChange::Created);
	});
}

void FRemoteSessionRole::CreateChannels(const TArray<FRemoteSessionChannelInfo>& InChannels)
{
	ClearChannels();

	if (OSCConnection != nullptr)
	{	
		for (const FRemoteSessionChannelInfo& Channel : InChannels)
		{
			OpenChannel(Channel);
		}
	}
}

void FRemoteSessionRole::ClearChannels()
{
	Channels.Empty();
}

TSharedPtr<IRemoteSessionChannel> FRemoteSessionRole::GetChannel(const TCHAR* InType)
{
	TSharedPtr<IRemoteSessionChannel> Channel;

	TSharedPtr<IRemoteSessionChannel>* FoundChannel = Channels.FindByPredicate([InType](const auto& Item) {
		const TCHAR* ItemType = Item->GetType();
		return FCString::Stricmp(ItemType, InType) == 0;
	});

	if (FoundChannel)
	{
		Channel = *FoundChannel;
	}

	return Channel;
}


bool FRemoteSessionRole::OpenChannel(const FRemoteSessionChannelInfo& Info)
{
	bool ChannelExists = Channels.ContainsByPredicate([&Info](auto& Channel) {
		return Channel->GetType() == Info.Type;
	});

	if (ChannelExists)
	{
		UE_LOGF(LogRemoteSession, Warning, "OpenChannel: OpenChannel called for %ls that already exists", *Info.Type);
		return false;
	}

	bool IsSupported = SupportedChannels.Num() == 0 || SupportedChannels.ContainsByPredicate([&Info](FRemoteSessionChannelInfo& Elem) {
		return Elem.Type == Info.Type;
		});

	if (!IsSupported)
	{
		UE_LOGF(LogRemoteSession, Display, "OpenChannel: OpenChannel called for %ls is not supported", *Info.Type);
		return false;
	}
		
	// Try to create the channel
	TSharedPtr<IRemoteSessionChannel> NewChannel;
	IRemoteSessionModule& RemoteSession = FModuleManager::GetModuleChecked<IRemoteSessionModule>("RemoteSession");

	NewChannel = FRemoteSessionChannelRegistry::Get().CreateChannel(*Info.Type, Info.Mode, OSCConnection);

	// If it's valid, send a notice to our peer that we opened this channel
	if (NewChannel.IsValid())
	{
		FString ModeString = ::LexToString(Info.Mode);

		UE_LOGF(LogRemoteSession, Log, "OpenChannel: Created Channel %ls with mode %ls", *Info.Type, *ModeString);
		Channels.Add(NewChannel);

		TBackChannelSharedPtr<IBackChannelPacket> Packet = OSCConnection->CreatePacket();

		Packet->SetPath(kChangeChannelEndPoint);			

		// We want our peer to open a channel with the opposite mode to us.
		ERemoteSessionChannelMode RequiredMode = Info.Mode == ERemoteSessionChannelMode::Read ? ERemoteSessionChannelMode::Write : ERemoteSessionChannelMode::Read;
		FString RequiredModeString = ::LexToString(RequiredMode);

		Packet->Write(TEXT("ChannelName"), Info.Type);
		Packet->Write(TEXT("ChannelMode"), RequiredModeString);
		Packet->Write(TEXT("Enabled"), 1);

		UE_LOGF(LogRemoteSession, Log, "OpenChannel: Requesting channel %ls with mode %ls from peer", *Info.Type, *RequiredModeString);

		OSCConnection->SendPacket(Packet);
	}
	else
	{
		UE_LOGF(LogRemoteSession, Error, "OpenChannel: Requested Channel %ls was not recognized", *Info.Type);
	}		

	return true;
}

void FRemoteSessionRole::OnReceiveChangeChannel(IBackChannelPacket& Message)
{
	FString ChannelName;
	FString ChannelMode;
	int32 Enabled(0);

	Message.Read(TEXT("Name"), ChannelName);
	Message.Read(TEXT("Mode"), ChannelMode);
	Message.Read(TEXT("Enabled"), Enabled);

	ERemoteSessionChannelMode Mode = ERemoteSessionChannelMode::Unknown;
	::LexFromString(Mode, *ChannelMode);

	if (ChannelName.Len() == 0 || Mode == ERemoteSessionChannelMode::Unknown)
	{
		UE_LOGF(LogRemoteSession, Error, "OnReceiveChangeChannel: Invalid channel details. Name=%ls, Mode=%ls", *ChannelName, *ChannelMode);
	}
	else
	{
		TSharedPtr<IRemoteSessionChannel>* ExistingChannel = Channels.FindByPredicate([&ChannelName](auto& Channel) {
			return Channel->GetType() == ChannelName;
		});

		if (ExistingChannel != nullptr && ExistingChannel->IsValid())
		{
			// this is a response to us creating a channe; so broadcast the delegate now it exists at both ends
			UE_LOGF(LogRemoteSession, Log, "OnReceiveChangeChannel: Channel %ls exists. Assuming peer response and broadcasting channel creation with mode %ls", *ChannelName, *ChannelMode);
			ChannelChangeDelegate.Broadcast(this, *ExistingChannel, ERemoteSessionChannelChange::Created);
		}
		else
		{				
			// Need to create channels on the main thread
			AsyncTask(ENamedThreads::GameThread, [this, ChannelName, ChannelMode, Mode] {

				UE_LOGF(LogRemoteSession, Log, "OnReceiveChangeChannel: Received request for %ls with mode=%ls", *ChannelName, *ChannelMode);
				FRemoteSessionChannelInfo Info(ChannelName, Mode);
				
				if (OpenChannel(Info))
				{
					TSharedPtr<IRemoteSessionChannel> Channel = GetChannel(*ChannelName);
					check(Channel.IsValid());
					// Since we received a request the channel should exist on the other end and we can send to it now.
					UE_LOGF(LogRemoteSession, Log, "OnReceiveChangeChannel: Broadcasting channel %ls creation with mode %ls", *ChannelName, *ChannelMode);
					ChannelChangeDelegate.Broadcast(this, Channel, ERemoteSessionChannelChange::Created);
				}
			});	
		}

	}
}


/* Queues the next state to be processed on the next tick. It's an error to call this when there is another state pending */
void FRemoteSessionRole::SetPendingState(const ConnectionState InState)
{
	checkf(PendingState == ConnectionState::Unknown, TEXT("PendingState must be unknown when SetPendingState is called"));
	PendingState = InState;
}

