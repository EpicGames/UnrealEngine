// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BackChannel/Types.h"
#include "IRemoteSessionRole.h"

class IBackChannelConnection;
class IBackChannelSocketConnection;

class FBackChannelOSCConnection;
enum class ERemoteSessionChannelMode;

class FRemoteSessionRole : public IRemoteSessionUnmanagedRole
{
protected:
	enum class ConnectionState
	{
		Unknown,
		Disconnected,
		UnversionedConnection,
		EstablishingVersion,
		Connected
	};

	const TCHAR* LexToString(ConnectionState InState)
	{
		switch (InState)
		{
		case ConnectionState::Unknown:
			return TEXT("Unknown");
		case ConnectionState::Disconnected:
			return TEXT("Disconnected");
		case ConnectionState::UnversionedConnection:
			return TEXT("UnversionedConnection");
		case ConnectionState::EstablishingVersion:
			return TEXT("EstablishingVersion");
		case ConnectionState::Connected:
			return TEXT("Connected");
		default:
			check(false);
			return TEXT("Unknown");
		}
	}

	const TCHAR* kLegacyVersionEndPoint = TEXT("/Version");
	const TCHAR* kLegacyChannelSelectionEndPoint = TEXT("/ChannelSelection");

	const TCHAR* kHelloEndPoint = TEXT("/RS.Hello");
	const TCHAR* kHelloV2EndPoint = TEXT("/RS.V2Hello");
	const TCHAR* kGoodbyeEndPoint = TEXT("/RS.Goodbye");
	const TCHAR* kChannelListEndPoint = TEXT("/RS.ChannelList");
	const TCHAR* kChangeChannelEndPoint = TEXT("/RS.ChangeChannel");
	const TCHAR* kPingEndPoint = TEXT("/RS.Ping");
	const TCHAR* kPongEndPoint = TEXT("/RS.Pong");

public:

	FRemoteSessionRole();
	virtual ~FRemoteSessionRole();
	
	virtual void Close(const FString& Message) override;

	virtual bool IsConnected() const override;
	
	virtual bool HasError() const override;
	
	virtual FString GetErrorMessage() const override;

	virtual void Tick( float DeltaTime ) override;

	/* Registers a delegate for notifications of connection changes*/
	virtual FDelegateHandle RegisterConnectionChangeDelegate(FOnRemoteSessionConnectionChange::FDelegate InDelegate) override;

	/* Register for notifications when the host sends a list of available channels */
	virtual FDelegateHandle RegisterChannelListDelegate(FOnRemoteSessionReceiveChannelList::FDelegate InDelegate) override;

	/* Register for notifications whenever a change in the state of a channel occurs */
	virtual FDelegateHandle RegisterChannelChangeDelegate(FOnRemoteSessionChannelChange::FDelegate InDelegate) override;

	/* Register for notifications whenever a hello route message arrives */
	virtual void RegisterHelloRouteMessageDelegate(FBackChannelRouteDelegate::FDelegate InDelegate) override;

	/* Unregister all delegates for the specified object */
	virtual void RemoveAllDelegates(FDelegateUserObject UserObject) override;

	virtual TSharedPtr<IRemoteSessionChannel> GetChannel(const TCHAR* Type) override;

	virtual bool OpenChannel(const FRemoteSessionChannelInfo& Info) override;

	virtual bool IsLegacyConnection() const override;

	virtual FString GetRemoteVersion() const override { return RemoteVersion; }

	virtual bool IsUsingPixelStreaming() const override { return bIsUsingPixelStreaming; }

	virtual FString GetRemotePixelStreamingVersion() const override { return RemotePixelStreamingVersion; }

	virtual FString GetRemoteBuildInfo() const override { return RemoteBuildInfo; }

	virtual float GetPeerResponseInSeconds() const { return SecondsForPeerResponse; }

protected:

	/* Closes all connections. Called by public Close() function which first send a graceful goodbye */
	virtual void	CloseConnections();

	/* Similar (and calls) Close(Message), but marks us as having closed due to an error */
	void			CloseWithError(const FString& Message);

	void			CreateOSCConnection(TSharedRef<IBackChannelSocketConnection> InConnection);
	
	void			SendLegacyVersionCheck();

	virtual void 	BindEndpoints(TBackChannelSharedPtr<IBackChannelConnection> InConnection);

	void 			OnReceiveLegacyVersion(IBackChannelPacket& Message);

	void			SendHello();
	void 			OnReceiveHello(IBackChannelPacket& Message);

	void			SendHelloV2();
	void 			OnReceiveHelloV2(IBackChannelPacket& Message);

	void			SendGoodbye(const FString& InMessage);
	void 			OnReceiveGoodbye(IBackChannelPacket& Message);

	void			SendPing();
	void 			OnReceivePing(IBackChannelPacket& Message);
	void 			OnReceivePong(IBackChannelPacket& Message);
		
	void 			CreateChannels(const TArray<FRemoteSessionChannelInfo>& Channels);

	/* 
		Called when the other side wants to open a channel. The connection that is the original requester will receive this message
		will receive this message when the other end has created the channel so the successful connection state can be broadcast
	*/
	void			OnReceiveChangeChannel(IBackChannelPacket& Message);

	void			OnReceiveChannelChanged(IBackChannelPacket& Message);
	
	void	        ClearChannels();

	/* Children of RemoteSessionRole can use this function to extend the information that a package will carry */
	virtual void	AppendExtraInfoToBackChannelPacket(TBackChannelSharedPtr<IBackChannelPacket> Packet) {};

	/* Queues the next state to be processed on the next tick. It's an error to call this when there is another state pending */
	void			SetPendingState(const ConnectionState InState);

	/*
		Called from the tick loop to perform any state changes. When called GetCurrentState() will return the current state. If the function 
		returnstrue  CurrentState will be set to IncomingState. If not the connection will be disconnected
	*/
	virtual bool	ProcessStateChange(const ConnectionState NewState, const ConnectionState OldState) = 0;

	/* Returns the current processed state */
	ConnectionState GetCurrentState(void) const { return CurrentState; }

	bool			IsStateCurrentOrPending(ConnectionState InState) const { return CurrentState == InState || PendingState == InState; }


protected:

	mutable FCriticalSection			CriticalSectionForMainThread;

	TSharedPtr<IBackChannelSocketConnection>	Connection;

	TSharedPtr<FBackChannelOSCConnection, ESPMode::ThreadSafe> OSCConnection;

	FOnRemoteSessionChannelChange ChannelChangeDelegate;

	FOnRemoteSessionConnectionChange ConnectionChangeDelegate;

	FOnRemoteSessionReceiveChannelList ReceiveChannelListDelegate;


	TArray<FRemoteSessionChannelInfo> SupportedChannels;

	bool					bIsUsingPixelStreaming;

	FString					RemotePixelStreamingVersion;

	FString					RemoteBuildInfo;

private:
	void			RemoveRouteDelegates();


	FString					RemoteVersion;

	FString					ErrorMessage;

	TSharedPtr<struct FRemoteSessionRoleCancellationToken>  CancelCloseToken;

	TArray<TSharedPtr<IRemoteSessionChannel>> Channels;

	TWeakPtr<IBackChannelConnection, ESPMode::ThreadSafe> BackChannelConnection;

	struct FRouteDelegates
	{
		FDelegateHandle Hello;
		FDelegateHandle HelloV2;
		FDelegateHandle Goodbye;
		FDelegateHandle Ping;
		FDelegateHandle Pong;
		FDelegateHandle LegacyReceive;
		FDelegateHandle ChangeChannel;
	};
	FRouteDelegates			RouteDelegates;

	TArray<FBackChannelRouteDelegate::FDelegate> ExternalHelloRouteDelegates;
	TArray<FDelegateHandle> ExternalHelloRouteDelegateHandles;

	ConnectionState			CurrentState = ConnectionState::Disconnected;

	ConnectionState			PendingState = ConnectionState::Unknown;

	/* Time since last ping */
	double				LastPingTime = 0;

	/* Time since last response */
	double				LastReponseTime = 0;

	/* Latency based on most recent ping/ping */
	float				SecondsForPeerResponse = 0;
};
