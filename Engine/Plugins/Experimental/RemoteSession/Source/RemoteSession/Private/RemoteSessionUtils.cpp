// Copyright Epic Games, Inc. All Rights Reserved.

#include "RemoteSessionUtils.h"
#include "Engine/GameEngine.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/SWindow.h"

#if WITH_EDITOR
	#include "Editor.h"
	#include "IAssetViewport.h"
#endif

void FRemoteSessionUtils::FindSceneViewport(TWeakPtr<SWindow>& OutInputWindow, TWeakPtr<FSceneViewport>& OutSceneViewport)
{
#if WITH_EDITOR
	if (GIsEditor)
	{
		for (const FWorldContext& Context : GEngine->GetWorldContexts())
		{
			if (Context.WorldType == EWorldType::PIE)
			{
				FSlatePlayInEditorInfo* SlatePlayInEditorSession = GEditor->SlatePlayInEditorMap.Find(Context.ContextHandle);
				if (SlatePlayInEditorSession)
				{
					if (SlatePlayInEditorSession->DestinationSlateViewport.IsValid())
					{
						TSharedPtr<IAssetViewport> DestinationLevelViewport = SlatePlayInEditorSession->DestinationSlateViewport.Pin();
						OutSceneViewport = DestinationLevelViewport->GetSharedActiveViewport();
						OutInputWindow = FSlateApplication::Get().FindWidgetWindow(DestinationLevelViewport->AsWidget());
					}
					else if (SlatePlayInEditorSession->SlatePlayInEditorWindowViewport.IsValid())
					{
						OutSceneViewport = SlatePlayInEditorSession->SlatePlayInEditorWindowViewport;
						OutInputWindow = SlatePlayInEditorSession->SlatePlayInEditorWindow;
					}
				}
			}
		}
	}
	else
#endif
	{
		UGameEngine* GameEngine = Cast<UGameEngine>(GEngine);
		OutSceneViewport = GameEngine->SceneViewport;
		OutInputWindow = GameEngine->GameViewportWindow;
	}
}
