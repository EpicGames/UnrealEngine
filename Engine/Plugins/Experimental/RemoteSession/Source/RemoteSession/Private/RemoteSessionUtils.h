// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

class SWindow;
class FSceneViewport;


struct FRemoteSessionUtils
{
	static void FindSceneViewport(TWeakPtr<SWindow>& OutInputWindow, TWeakPtr<FSceneViewport>& OutSceneViewport);
};
