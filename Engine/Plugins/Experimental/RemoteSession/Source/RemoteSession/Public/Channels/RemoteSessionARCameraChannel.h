// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RemoteSessionChannel.h"
#include "UObject/GCObject.h"

#define UE_API REMOTESESSION_API

enum class ERemoteSessionChannelMode : int32;

class FBackChannelOSCMessage;
class FBackChannelOSCDispatch;
class FAppleImageUtilsConversionTaskBase;
class UTexture2D;
class UMaterialInterface;
class UMaterialInstanceDynamic;
class ISceneViewExtension;
class IBackChannelPacket;

class FRemoteSessionARCameraChannel :
	public IRemoteSessionChannel,
	public FGCObject
{
public:

	UE_API FRemoteSessionARCameraChannel(ERemoteSessionChannelMode InRole, TSharedPtr<IBackChannelConnection, ESPMode::ThreadSafe> InConnection);

	UE_API ~FRemoteSessionARCameraChannel();

	UE_API virtual void Tick(const float InDeltaTime) override;

	/* Begin IRemoteSessionChannel implementation */
	static const TCHAR* StaticType() { return TEXT("FRemoteSessionARCameraChannel"); }
	virtual const TCHAR* GetType() const override { return StaticType(); }
	/* End IRemoteSessionChannel implementation */

	/** Returns the post process material to render with. The textures are set on the fly */
	UE_API UMaterialInterface* GetPostProcessMaterial() const;

protected:
	/** Sends the most recent compressed AR camera image to the remote */
	UE_API void SendARCameraImage();
	/** Compresses in the background and queues for sending */
	UE_API void QueueARCameraImage();

	/** Handles data coming from the client */
	UE_API void ReceiveARCameraImage(IBackChannelPacket& Message);

	/** Updates the next texture to be rendered */
	UE_API void UpdateRenderingTexture();

	//~ FGCObject
	UE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FRemoteSessionARCameraChannel");
	}
	//~ FGCObject

private:
	/** A wrapper around the async compression task */
	struct FCompressionTask
	{
		FCompressionTask() : Width(0), Height(0) {}

		int32 Width;
		int32 Height;
		TSharedPtr<FAppleImageUtilsConversionTaskBase, ESPMode::ThreadSafe> AsyncTask;
	};

	struct FDecompressedImage
	{
		FDecompressedImage() : Width(0), Height(0) {}

		int32 Width;
		int32 Height;
		TArray<uint8> ImageData;
	};

	/** Only compress one frame at a time to prevent flooding the GPU */
	TSharedPtr<FCompressionTask, ESPMode::ThreadSafe> CompressionTask;

	/** Images that have been received and are available for rendering */
	TArray<TSharedPtr<FDecompressedImage, ESPMode::ThreadSafe>> DecompressionQueue;
	FThreadSafeCounter DecompressionTaskCount;
	FCriticalSection DecompressionQueueLock;

	/** The post process material that the overlay will render with */
	TObjectPtr<UMaterialInterface> PPMaterial;
	/** To set the current texture for rendering */
	TObjectPtr<UMaterialInstanceDynamic> MaterialInstanceDynamic;
	/** Textures to use when rendering */
	TObjectPtr<UTexture2D> RenderingTextures[2];
	UTexture2D* LastSetTexture;
	FThreadSafeCounter RenderingTextureIndex;
	FThreadSafeCounter RenderingTexturesUpdateCount[2];

	TSharedPtr<ISceneViewExtension, ESPMode::ThreadSafe> SceneViewExtension;

	TSharedPtr<IBackChannelConnection, ESPMode::ThreadSafe> Connection;
	ERemoteSessionChannelMode Role;

	/** So we can manage callback lifetimes properly */
	FDelegateHandle MessageCallbackHandle;
};

class FRemoteSessionARCameraChannelFactoryWorker : public IRemoteSessionChannelFactoryWorker
{
public:
	UE_API virtual TSharedPtr<IRemoteSessionChannel> Construct(ERemoteSessionChannelMode InMode, TSharedPtr<IBackChannelConnection, ESPMode::ThreadSafe> InConnection) const override;
};

#undef UE_API
