// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Channels/RemoteSessionChannel.h"
#include "RemoteSessionXRTrackingChannel.h"
#include "ARSystem.h"
#include "ARTraceResult.h"
#include "UObject/GCObject.h"

#define UE_API REMOTESESSION_API

class UARTextureCameraDepth;
class UARTextureCameraImage;

class IBackChannelPacket;

/**
 * This class acts as if there is a AR system present on the desktop that is receiving data from a remote device
 */
class FARSystemProxy :
	public IARSystemSupport,
	public FGCObject,
	public TSharedFromThis<FARSystemProxy, ESPMode::ThreadSafe>
{
public:
	/** Only the factory method should construct this */
	FARSystemProxy();
	virtual ~FARSystemProxy();

	static TSharedPtr<FARSystemProxy, ESPMode::ThreadSafe> Get();
	static IARSystemSupport* GetARSystemPtr();
	static void Destroy();

	// ~IARSystemSupport
	
	/** Returns true/false based on whether AR features are available */
	virtual bool IsARAvailable() const override;
	
	virtual EARTrackingQuality OnGetTrackingQuality() const override;
	/** @return the reason of limited tracking quality; if the state is not limited, return EARTrackingQualityReason::None */
	virtual EARTrackingQualityReason OnGetTrackingQualityReason() const override;
	virtual FARSessionStatus OnGetARSessionStatus() const override;
	virtual TArray<UARTrackedGeometry*> OnGetAllTrackedGeometries() const override;
	virtual bool OnIsTrackingTypeSupported(EARSessionType SessionType) const override;
	virtual EARWorldMappingState OnGetWorldMappingStatus() const override;
	virtual TArray<FARVideoFormat> OnGetSupportedVideoFormats(EARSessionType SessionType) const override;
	// Not supported methods
	// @todo JoeG -- Look at supporting this
	virtual UARLightEstimate* OnGetCurrentLightEstimate() const override { return nullptr; }
	virtual UARTextureCameraImage* OnGetCameraImage() { return nullptr; }
	virtual UARTextureCameraDepth* OnGetCameraDepth() { return nullptr; }
	// End todo block
	virtual void OnStartARSession(UARSessionConfig* Config) override {}
	virtual void OnPauseARSession() override {}
	virtual void OnStopARSession() override {}
	virtual void OnSetAlignmentTransform(const FTransform& InAlignmentTransform) override {}
	virtual bool OnAddManualEnvironmentCaptureProbe(FVector Location, FVector Extent) override { return false; }
	virtual TSharedPtr<FARSaveWorldAsyncTask, ESPMode::ThreadSafe> OnSaveWorld() const override
	{
		return MakeShared<FARErrorSaveWorldAsyncTask, ESPMode::ThreadSafe>(TEXT("Not supported on proxy"));
	}
	virtual TSharedPtr<FARGetCandidateObjectAsyncTask, ESPMode::ThreadSafe> OnGetCandidateObject(FVector Location, FVector Extent) const override
	{
		return MakeShared<FARErrorGetCandidateObjectAsyncTask, ESPMode::ThreadSafe>(TEXT("Not supported on proxy"));
	}
	virtual bool OnAddRuntimeCandidateImage(UARSessionConfig* Config, UTexture2D* CandidateTexture, FString FriendlyName, float PhysicalWidth) override { return false; }
	virtual TArray<FVector> OnGetPointCloud() const override { return TArray<FVector>(); }
	virtual TArray<UARPin*> OnGetAllPins() const override { return TArray<UARPin*>(); }
	virtual UARPin* OnPinComponent(USceneComponent* ComponentToPin, const FTransform& PinToWorldTransform, UARTrackedGeometry* TrackedGeometry = nullptr, const FName DebugName = NAME_None) { return nullptr; }
	virtual void OnRemovePin(UARPin* PinToRemove) override {}
	virtual UARPin* FindARPinByComponent(const USceneComponent* Component) const override { return nullptr; }
	virtual bool OnPinComponentToARPin(USceneComponent* ComponentToPin, UARPin* Pin) override { return true; }
	virtual TArray<FARTraceResult> OnLineTraceTrackedObjects( const FVector2D ScreenCoord, EARLineTraceChannels TraceChannels ) override { return TArray<FARTraceResult>(); }
	virtual TArray<FARTraceResult> OnLineTraceTrackedObjects( const FVector Start, const FVector End, EARLineTraceChannels TraceChannels ) override { return TArray<FARTraceResult>(); }
	virtual void* GetARSessionRawPointer() override { return nullptr; }
	virtual void* GetGameThreadARFrameRawPointer() override { return nullptr; }
	// ~IARSystemSupport

	//~ FGCObject
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FARSystemProxy");
	}
	//~ FGCObject

	// Methods called by the message handlers
	void SetSupportedVideoFormats(const TArray<FARVideoFormat>& InFormats);
	void SetSessionConfig(UARSessionConfig* InConfig);
	void AddTrackable(UARTrackedGeometry* Added);
	UARTrackedGeometry* GetTrackable(FGuid UniqueId);
	void NotifyUpdated(UARTrackedGeometry* Updated);
	void RemoveTrackable(FGuid UniqueId);
	// ~Methods called by the message handlers

private:
	static TSharedPtr<FARSystemProxy, ESPMode::ThreadSafe> FactoryInstance;

	/** Used to return the session config object the remote is using */
	TObjectPtr<UARSessionConfig> SessionConfig;
	/** Map of unique ids to tracked geometries */
	TMap<FGuid, TObjectPtr<UARTrackedGeometry>> TrackedGeometries;
	/** The video formats that are supported */
	TArray<FARVideoFormat> SupportedFormats;
};

/**
 * Class used to deliver AR trackable items to the remote machine. Inherits from the XR tracking
 * channel which handles sending all of the position & orientation updates. Do not use both this
 * channel and the XR tracking at the same time, since they both hook GEngine->XRSystem and
 * you will get inconsistent results
 */
class FRemoteSessionARSystemChannel :
	public FRemoteSessionXRTrackingChannel
{
public:
	UE_API FRemoteSessionARSystemChannel(ERemoteSessionChannelMode InRole, TSharedPtr<IBackChannelConnection, ESPMode::ThreadSafe> InConnection);

	UE_API virtual ~FRemoteSessionARSystemChannel();

	/* Begin IRemoteSessionChannel implementation */
	static const TCHAR* StaticType() { return TEXT("FRemoteSessionARSystemChannel"); }
	virtual const TCHAR* GetType() const override { return StaticType(); }
	/* End IRemoteSessionChannel implementation */

	// Message handlers
	UE_API void ReceiveARInit(IBackChannelPacket& Message);
	UE_API void ReceiveAddTrackable(IBackChannelPacket& Message);
	UE_API void ReceiveUpdateTrackable(IBackChannelPacket& Message);
	UE_API void ReceiveRemoveTrackable(IBackChannelPacket& Message);

	// Game thead message handlers
	UE_API void ReceiveARInit_GameThread(FString ConfigObjectPathName, TArray<FARVideoFormat> Formats);
	UE_API void ReceiveAddTrackable_GameThread(FString ClassPathName, TSharedPtr<TArray<uint8>, ESPMode::ThreadSafe> DataCopy);
	UE_API void ReceiveUpdateTrackable_GameThread(FGuid UniqueId, TSharedPtr<TArray<uint8>, ESPMode::ThreadSafe> DataCopy);
	UE_API void ReceiveRemoveTrackable_GameThread(FGuid UniqueId);

	// AR notification handlers that send the data to the listener
	UE_API void SendAddedMessage(UARTrackedGeometry* Added);
	UE_API void SendUpdatedMessage(UARTrackedGeometry* Updated);
	UE_API void SendRemovedMessage(UARTrackedGeometry* Removed);

	/** Sends the name of the config object we're using so it can be loaded on the remote. Also passes the video settings and supported session types */
	UE_API void SendARInitMessage();

private:
	/** The buffer that objects are serialized into. Cached to reduce malloc/free pressure */
	TArray<uint8> SerializeBuffer;

	/** Channel message handles */
	FDelegateHandle InitMessageCallbackHandle;
	FDelegateHandle AddMessageCallbackHandle;
	FDelegateHandle UpdateMessageCallbackHandle;
	FDelegateHandle RemoveMessageCallbackHandle;

	/** AR system notification handles */
	FDelegateHandle OnTrackableAddedDelegateHandle;
	FDelegateHandle OnTrackableUpdatedDelegateHandle;
	FDelegateHandle OnTrackableRemovedDelegateHandle;
};

class FRemoteSessionARSystemChannelFactoryWorker : public IRemoteSessionChannelFactoryWorker
{
public:
	UE_API virtual TSharedPtr<IRemoteSessionChannel> Construct(ERemoteSessionChannelMode InMode, TSharedPtr<IBackChannelConnection, ESPMode::ThreadSafe> InConnection) const override;
};

#undef UE_API
