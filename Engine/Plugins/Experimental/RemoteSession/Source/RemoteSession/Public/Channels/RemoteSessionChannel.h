// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BackChannel/Types.h"
#include "RemoteSessionTypes.h"

#define UE_API REMOTESESSION_API

class IBackChannelConnection;

class IRemoteSessionChannel
{
public:

	IRemoteSessionChannel(ERemoteSessionChannelMode InRole, TBackChannelSharedPtr<IBackChannelConnection> InConnection) {}

	virtual ~IRemoteSessionChannel() = default;

	virtual void Tick(const float InDeltaTime) = 0;

	virtual const TCHAR* GetType() const = 0;
};

class IRemoteSessionChannelFactoryWorker
{
public:
	virtual ~IRemoteSessionChannelFactoryWorker() = default;
	virtual TSharedPtr<IRemoteSessionChannel> Construct(ERemoteSessionChannelMode InMode, TBackChannelSharedPtr<IBackChannelConnection> InConnection) const = 0;
};


class FRemoteSessionChannelRegistry
{
public:
	static FRemoteSessionChannelRegistry& Get()
	{
		static FRemoteSessionChannelRegistry Instance;
		return Instance;
	}

	UE_API void RegisterChannelFactory(const TCHAR* InChannelName, ERemoteSessionChannelMode InHostMode, TWeakPtr<IRemoteSessionChannelFactoryWorker> InFactory);
	UE_API void RemoveChannelFactory(TWeakPtr<IRemoteSessionChannelFactoryWorker> InFactory);

	UE_API TSharedPtr<IRemoteSessionChannel> CreateChannel(const FStringView InChannelName, ERemoteSessionChannelMode InMode, TBackChannelSharedPtr<IBackChannelConnection> InConnection);
	UE_API const TArray<FRemoteSessionChannelInfo>& GetRegisteredFactories() const;

protected:

	FRemoteSessionChannelRegistry() = default;
	TMap<FString, TWeakPtr<IRemoteSessionChannelFactoryWorker>> RegisteredFactories;
	TArray<FRemoteSessionChannelInfo> KnownChannels;
};


#define REGISTER_CHANNEL_FACTORY(ChannelName, FactoryClass, HostMode ) \
	class AutoRegister_##ChannelName { \
	public: \
		AutoRegister_##ChannelName() \
		{ \
			static auto Factory =  MakeShared<FactoryClass>(); \
			FRemoteSessionChannelRegistry::Get().RegisterChannelFactory(TEXT(#ChannelName), HostMode, Factory); \
		} \
	}; \
	AutoRegister_##ChannelName G##ChannelName

#undef UE_API
