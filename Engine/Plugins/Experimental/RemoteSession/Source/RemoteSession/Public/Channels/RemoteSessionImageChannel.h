// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RemoteSessionChannel.h"
#include "HAL/Runnable.h"
#include "HAL/ThreadSafeBool.h"

#define UE_API REMOTESESSION_API

enum class ERemoteSessionChannelMode : int32;


class FBackChannelOSCMessage;
class FBackChannelOSCDispatch;
class FRemoteSessionImageChannel;
class IRemoteSessionImageProvider;
class UTexture2D;
class IBackChannelPacket;

class IRemoteSessionImageProvider
{
public:
	virtual ~IRemoteSessionImageProvider() = default;
	virtual void Tick(const float InDeltaTime) = 0;
};


DECLARE_STATS_GROUP(TEXT("RemoteSession"), STATGROUP_RemoteSession, STATCAT_Advanced);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("RS.CapturedFrames/s"), STAT_RSCaptureCount, STATGROUP_RemoteSession);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("RS.SkippedFrames/s"), STAT_RSSkippedFrames, STATGROUP_RemoteSession);
DECLARE_CYCLE_STAT(TEXT("RS.ImageCaptureTime"), STAT_RSCaptureTime, STATGROUP_RemoteSession);
DECLARE_CYCLE_STAT(TEXT("RS.ImageCompressTime"), STAT_RSCompressTime, STATGROUP_RemoteSession);

// Receiving counters and stats
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("RS.WaitingFrames/s"), STAT_RSWaitingFrames, STATGROUP_RemoteSession);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("RS.DiscardedFrames/s"), STAT_RSDiscardedFrames, STATGROUP_RemoteSession);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("RS.MaxImageProcessTime"), STAT_RSMaxImageProcessTime, STATGROUP_RemoteSession);
DECLARE_CYCLE_STAT(TEXT("RS.ReceiveTime"), STAT_RSReceiveTime, STATGROUP_RemoteSession);
DECLARE_CYCLE_STAT(TEXT("RS.WakeupWait"), STAT_RSWakeupWait, STATGROUP_RemoteSession);
DECLARE_CYCLE_STAT(TEXT("RS.ImageDecompressTime"), STAT_RSDecompressTime, STATGROUP_RemoteSession);
DECLARE_CYCLE_STAT(TEXT("RS.TextureUpdate"), STAT_RSTextureUpdate, STATGROUP_RemoteSession);
DECLARE_CYCLE_STAT(TEXT("RS.TickRate"), STAT_RSTickRate, STATGROUP_RemoteSession);
DECLARE_CYCLE_STAT(TEXT("RSReadyFrameCount"), STAT_RSNumFrames, STATGROUP_RemoteSession);


struct FRemoteSesstionImageCaptureStats
{
	int32	FramesCaptured = 0;
	int32	FramesSkipped = 0;
	double	LastUpdateTime = 0;
};


struct FRemoteSessionImageReceiveStats
{
	int32	FramesWaiting = 0;
	int32	FramesSkipped = 0;
    double  MaxImageProcessTime = 0;
	double	LastUpdateTime = 0;
};


/**
 *	A channel that takes an image (created by an IRemoteSessionImageProvider), then sends it to the client (with a FRemoteSessionImageSender).
 *	On the client images are decoded into a double-buffered texture that can be accessed via GetHostScreen.
 */
class FRemoteSessionImageChannel : public IRemoteSessionChannel, FRunnable
{
public:

	/**
	*	A helper object responsible to take the raw data, encode it to jpg and send it to the client for the RemoteSessionImageChannel
	*/
	class FImageSender
	{
	public:
  
		FImageSender(TSharedPtr<IBackChannelConnection, ESPMode::ThreadSafe> InConnection);
  
		/** Set the jpg compression quality */
		void SetCompressQuality(int32 InQuality);
  
		/** Send an image to the connected clients */
		void SendRawImageToClients(int32 Width, int32 Height, const TArray<FColor>& ImageData, int32 BytesPerRow = 0);
  
		/** Send an image to the connected clients */
		void SendRawImageToClients(int32 Width, int32 Height, const void* ImageData, int32 AllocatedImageDataSize, int32 BytesPerRow = 0);
  
	private:
  
		/** Underlying connection */
		TWeakPtr<IBackChannelConnection, ESPMode::ThreadSafe> Connection;
  
		/** Compression quality of the raw image we wish to send to client */
		TAtomic<int32> CompressQuality;
  
		int32 NumSentImages;
	};

public:

	UE_API FRemoteSessionImageChannel(ERemoteSessionChannelMode InRole, TSharedPtr<IBackChannelConnection, ESPMode::ThreadSafe> InConnection);

	UE_API ~FRemoteSessionImageChannel();

	/** Tick this channel */
	UE_API virtual void Tick(const float InDeltaTime) override;

	/** Get the client Texture2D to display */
	UE_API UTexture2D* GetHostScreen() const;

	/** Set the ImageProvider that will produce the images that will be sent to the client */
	UE_API void SetImageProvider(TSharedPtr<IRemoteSessionImageProvider> ImageProvider);

	/** Sets up an image provider that mirrors the games framebuffer. WIll be the default if no ImageProvider is set. */
	UE_API void SetFramebufferAsImageProvider();

	/** Set the jpg compression quality */
	UE_API void SetCompressQuality(int32 InQuality);

	/** Return the image sender connected to the clients */
	TSharedPtr<FRemoteSessionImageChannel::FImageSender, ESPMode::ThreadSafe> GetImageSender() { return ImageSender; }

	/* Begin IRemoteSessionChannel implementation */
	static const TCHAR* StaticType() { return TEXT("FRemoteSessionImageChannel"); }
	virtual const TCHAR* GetType() const override { return StaticType(); }
	/* End IRemoteSessionChannel implementation */

protected:

	/** Underlying connection */
	TWeakPtr<IBackChannelConnection, ESPMode::ThreadSafe> Connection;

	/** Our role */
	ERemoteSessionChannelMode Role;

	/** */
	TSharedPtr<IRemoteSessionImageProvider> ImageProvider;

	/** Bound to receive incoming images */
	UE_API void ReceiveHostImage(IBackChannelPacket& Message);

	/** Creates a texture to receive images into */
	UE_API void CreateTexture(const int32 InSlot, const int32 InWidth, const int32 InHeight);
	
	struct FImageData
	{
        int32				Width = 0;
        int32				Height = 0;
        int32				ImageIndex = 0;
        double              TimeCreated = 0;
        TArray<uint8>       ImageData;
	};

	FCriticalSection										IncomingImageMutex;
	TArray<TUniquePtr<FImageData>>							IncomingEncodedImages;

	FCriticalSection										DecodedImageMutex;
	TArray<TUniquePtr<FImageData>>							IncomingDecodedImages;

	UTexture2D*												DecodedTextures[2];
	TSharedPtr<TAtomic<int32>, ESPMode::ThreadSafe>			DecodedTextureIndex;

	/** Image sender used by the channel */
	TSharedPtr<FImageSender, ESPMode::ThreadSafe> ImageSender;

	/** So we can manage callback lifetimes properly */
	FDelegateHandle MessageCallbackHandle;

	/* Last processed incoming image */
	int32 LastIncomingImageIndex;

	/* Last processed decode image */
	int32 LastDecodedImageIndex;

	/** Decode the incoming images on a dedicated thread */
	UE_API void ProcessIncomingTextures();

protected:
	
	/** Background thread for decoding incoming images */
	UE_API uint32 Run();
	UE_API void StartBackgroundThread();
	UE_API void ExitBackgroundThread();

	FRunnableThread*	BackgroundThread;
	FEvent *			ScreenshotEvent;

	FThreadSafeBool		ExitRequested;

	bool				HaveConfiguredImageProvider;

	FRemoteSessionImageReceiveStats	ReceiveStats;
};

#undef UE_API
