// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RemoteSessionChannel.h"
#include "../Private/MessageHandler/RecordingMessageHandler.h"

#define UE_API REMOTESESSION_API

class FGenericApplicationMessageHandler;
enum class ERemoteSessionChannelMode : int32;

class FBackChannelOSCMessage;
class FBackChannelOSCDispatch;
class IBackChannelPacket;

class FRemoteSessionInputChannel : public IRemoteSessionChannel, public IRecordingMessageHandlerWriter
{
public:

	UE_API FRemoteSessionInputChannel(ERemoteSessionChannelMode InRole, TSharedPtr<IBackChannelConnection, ESPMode::ThreadSafe> InConnection);

	UE_API ~FRemoteSessionInputChannel();

	UE_API virtual void Tick(const float InDeltaTime) override;

	UE_API virtual void RecordMessage(const TCHAR* MsgName, const TArray<uint8>& Data) override;

	UE_API void OnRemoteMessage(IBackChannelPacket& Message);

	UE_API void SetPlaybackWindow(TWeakPtr<SWindow> InWindow, TWeakPtr<FSceneViewport> InViewport);

	UE_API void SetInputRect(const FVector2D& TopLeft, const FVector2D& Extents);

	UE_API void TryRouteTouchMessageToWidget(bool bRouteMessageToWidget);

	/**
	 * Delegate that fires when routing a touch message to the widget did not work.
	 * @note Only useful during playback.
	 * @return Null when recording, Pointer to delegate during playback.
	 */
	UE_API FOnRouteTouchDownToWidgetFailedDelegate* GetOnRouteTouchDownToWidgetFailedDelegate();

	static const TCHAR* StaticType() { return TEXT("FRemoteSessionInputChannel"); }
	virtual const TCHAR* GetType() const override { return StaticType(); }

protected:

	TSharedPtr<FGenericApplicationMessageHandler> DefaultHandler;

	TSharedPtr<FRecordingMessageHandler> RecordingHandler;

	TSharedPtr<FRecordingMessageHandler> PlaybackHandler;

	TSharedPtr<IBackChannelConnection, ESPMode::ThreadSafe> Connection;

	ERemoteSessionChannelMode Role;

	/** So we can manage callback lifetimes properly */
	FDelegateHandle MessageCallbackHandle;
};

#undef UE_API
