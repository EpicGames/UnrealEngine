// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RemoteSessionChannel.h"

#define UE_API REMOTESESSION_API

enum class ERemoteSessionChannelMode : int32;


class FBackChannelOSCDispatch;
class IBackChannelPacket;


class FRemoteSessionLiveLinkChannel :	public IRemoteSessionChannel
{
public:

	UE_API FRemoteSessionLiveLinkChannel(ERemoteSessionChannelMode InRole, TSharedPtr<IBackChannelConnection, ESPMode::ThreadSafe> InConnection);

	UE_API virtual ~FRemoteSessionLiveLinkChannel();

	UE_API virtual void Tick(const float InDeltaTime) override;

	/** Sends the current location and rotation for the XRTracking system to the remote */
	UE_API void SendLiveLinkHello(const FStringView InSubjectName);

	/* Begin IRemoteSessionChannel implementation */
	static const TCHAR* StaticType() { return TEXT("FRemoteSessionLiveLinkChannel"); }
	virtual const TCHAR* GetType() const override { return StaticType(); }
	/* End IRemoteSessionChannel implementation */

protected:
	

	/** Handles data coming from the client */
	UE_API void ReceiveLiveLinkHello(IBackChannelPacket& Message);

	TSharedPtr<IBackChannelConnection, ESPMode::ThreadSafe> Connection;

	ERemoteSessionChannelMode Role;

	/** So we can manage callback lifetimes properly */
	FDelegateHandle RouteHandle;

};

#undef UE_API
