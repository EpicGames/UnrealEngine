// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RemoteSessionChannel.h"
#include "XRTrackingSystemBase.h"

#define UE_API REMOTESESSION_API

class IXRTrackingSystem;
enum class ERemoteSessionChannelMode : int32;
enum class EXRTrackedDeviceType : uint8;

class FBackChannelOSCDispatch;
class IBackChannelPacket;

class FXRTrackingProxy :
	public FXRTrackingSystemBase
{
public:
	FXRTrackingProxy(IARSystemSupport* InARSystemSupport)
		: FXRTrackingSystemBase(InARSystemSupport)
	{}
		
	virtual bool IsTracking(int32 DeviceId) override { return true; }
	virtual bool DoesSupportPositionalTracking() const override { return true; }
	virtual bool IsHeadTrackingAllowed() const override { return true; }
	virtual void ResetOrientationAndPosition(float Yaw = 0.f) override { }
	virtual float GetWorldToMetersScale() const override { return 100.f; }

	UE_API virtual bool EnumerateTrackedDevices(TArray<int32>& OutDevices, EXRTrackedDeviceType Type) override;
	UE_API virtual bool GetCurrentPose(int32 DeviceId, FQuat& OutOrientation, FVector& OutPosition) override;
	UE_API virtual FName GetSystemName() const override;
	virtual int32 GetXRSystemFlags() const override { return 0; }
};

class FRemoteSessionXRTrackingChannel :
	public IRemoteSessionChannel
{
public:

	UE_API FRemoteSessionXRTrackingChannel(ERemoteSessionChannelMode InRole, TSharedPtr<IBackChannelConnection, ESPMode::ThreadSafe> InConnection);

	UE_API virtual ~FRemoteSessionXRTrackingChannel();

	UE_API virtual void Tick(const float InDeltaTime) override;

	/** Sends the current location and rotation for the XRTracking system to the remote */
	UE_API void SendXRTracking();

	/** Handles data coming from the client */
	UE_API void ReceiveXRTracking(IBackChannelPacket& Message);

	/* Begin IRemoteSessionChannel implementation */
	static const TCHAR* StaticType() { return TEXT("FRemoteSessionXRTrackingChannel"); }
	virtual const TCHAR* GetType() const override { return StaticType(); }
	/* End IRemoteSessionChannel implementation */

protected:
	/** Only to be called by child classes */
	UE_API FRemoteSessionXRTrackingChannel(ERemoteSessionChannelMode InRole, TSharedPtr<IBackChannelConnection, ESPMode::ThreadSafe> InConnection, IARSystemSupport* ARSystemSupport);

	TSharedPtr<IBackChannelConnection, ESPMode::ThreadSafe> Connection;
	ERemoteSessionChannelMode Role;

	/** If we're sending, this is GEngine->XRSystem. If we are receiving, this is the previous GEngine->XRSystem */
	TSharedPtr<IXRTrackingSystem, ESPMode::ThreadSafe> XRSystem;
	/** Used to set the values from the remote client as the XRTracking's pose */
	TSharedPtr<FXRTrackingProxy, ESPMode::ThreadSafe> ProxyXRSystem;

	/** Used by a child class to pass in a AR system to use as part of the proxy */
	IARSystemSupport* ARSystemSupport;

private:
	/** So we can manage callback lifetimes properly */
	FDelegateHandle MessageCallbackHandle;

	/** Used to finish construction of the class. Should be called from within the ctors */
	UE_API void Init();
};

#undef UE_API
