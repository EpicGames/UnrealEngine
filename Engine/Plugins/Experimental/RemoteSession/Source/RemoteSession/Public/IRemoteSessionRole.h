// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RemoteSessionTypes.h"
#include "BackChannel/Types.h"

class IRemoteSessionChannel;


class IRemoteSessionRole
{
public:
	virtual~IRemoteSessionRole() = default;

	virtual bool IsConnected() const = 0;
	
	virtual bool HasError() const = 0;
	
	virtual FString GetErrorMessage() const = 0;

	/* Future versions will support querying the version type, at the moment we just have old & new */
	virtual bool IsLegacyConnection() const = 0;

	/* Returns the version string reported by the remote peer during the Hello handshake. Empty before Connected. */
	virtual FString GetRemoteVersion() const = 0;

	/* True when PixelStreaming was negotiated as the streaming transport for this session. */
	virtual bool IsUsingPixelStreaming() const = 0;

	/* PixelStreaming version advertised by the remote peer (e.g. "PS2"). Empty when the peer did not advertise PS. */
	virtual FString GetRemotePixelStreamingVersion() const = 0;

	/* Opaque build identifier advertised by the remote peer in the Hello handshake. Empty when the peer did not advertise one. */
	virtual FString GetRemoteBuildInfo() const = 0;


	/* Registers a delegate for notifications of connection changes*/
	virtual FDelegateHandle RegisterConnectionChangeDelegate(FOnRemoteSessionConnectionChange::FDelegate InDelegate) = 0;

	/* Register for notifications when the host sends a list of available channels */
	virtual FDelegateHandle RegisterChannelListDelegate(FOnRemoteSessionReceiveChannelList::FDelegate InDelegate) = 0;

	/* Register for notifications whenever a change in the state of a channel occurs */
	virtual FDelegateHandle RegisterChannelChangeDelegate(FOnRemoteSessionChannelChange::FDelegate InDelegate) = 0;

	virtual void RegisterHelloRouteMessageDelegate(FBackChannelRouteDelegate::FDelegate InDelegate) = 0;

	/* Unregister all delegates for the specified object */
	virtual void RemoveAllDelegates(FDelegateUserObject UserObject) = 0;

	virtual TSharedPtr<IRemoteSessionChannel> GetChannel(const TCHAR* Type) = 0;

	template<class T>
	TSharedPtr<T> GetChannel()
	{
		TSharedPtr<IRemoteSessionChannel> Channel = GetChannel(T::StaticType());

		if (Channel.IsValid())
		{
			return StaticCastSharedPtr<T>(Channel);
		}

		return TSharedPtr<T>();
	}

	virtual bool OpenChannel(const FRemoteSessionChannelInfo& Info) = 0;

};

class IRemoteSessionUnmanagedRole : public IRemoteSessionRole
{
public:
	virtual void Tick(float DeltaTime) = 0;
	virtual void Close(const FString& InReason) = 0;
};
