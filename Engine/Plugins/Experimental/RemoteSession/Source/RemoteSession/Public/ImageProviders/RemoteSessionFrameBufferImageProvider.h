// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Channels/RemoteSessionImageChannel.h"

#define UE_API REMOTESESSION_API

class FThreadSafeCounter;


class FFrameGrabber;
class FSceneViewport;
class SWindow;

/**
 *	Use the FrameGrabber on the host to provide an image to the image channel.
 */
class FRemoteSessionFrameBufferImageProvider : public IRemoteSessionImageProvider
{
public:

	UE_API FRemoteSessionFrameBufferImageProvider(TSharedPtr<FRemoteSessionImageChannel::FImageSender, ESPMode::ThreadSafe> ImageSender);
	UE_API ~FRemoteSessionFrameBufferImageProvider();

	/** Specifies which viewport to capture */
	UE_API void SetCaptureViewport(TSharedRef<FSceneViewport> Viewport);

	/** Specifies the framerate at */
	UE_API void SetCaptureFrameRate(int32 InFramerate);

	/** Tick this channel */
	UE_API virtual void Tick(const float InDeltaTime) override;

	/** Signals that the viewport was resized */
	UE_API void OnViewportResized(FVector2D NewSize);

protected:

	/** Release the FrameGrabber */
	UE_API void ReleaseFrameGrabber();

	/** When the window is destroyed */
	UE_API void OnWindowClosedEvent(const TSharedRef<SWindow>&);

	/** Safely create the frame grabber */
	UE_API void CreateFrameGrabber(TSharedRef<FSceneViewport> Viewport);

	TWeakPtr<FRemoteSessionImageChannel::FImageSender, ESPMode::ThreadSafe> ImageSender;

	TSharedPtr<FFrameGrabber> FrameGrabber;

	TSharedPtr<FThreadSafeCounter, ESPMode::ThreadSafe> NumDecodingTasks;

	/** Time we last sent an image */
	double LastSentImageTime;

	/** Shows that the viewport was just resized */
	bool ViewportResized;

	/** Holds a reference to the scene viewport */
	TWeakPtr<FSceneViewport> SceneViewport;

	/** Holds a reference to the SceneViewport SWindow */
	TWeakPtr<SWindow> SceneViewportWindow;

	FRemoteSesstionImageCaptureStats CaptureStats;
};


#undef UE_API
