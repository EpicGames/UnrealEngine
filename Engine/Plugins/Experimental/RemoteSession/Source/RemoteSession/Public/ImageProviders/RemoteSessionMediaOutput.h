// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MediaCapture.h"
#include "RemoteSessionMediaOutput.generated.h"

#define UE_API REMOTESESSION_API

class FRemoteSessionImageChannel;

UCLASS(MinimalAPI, BlueprintType)
class URemoteSessionMediaOutput : public UMediaOutput
{
	GENERATED_BODY()

	//~ Begin UMediaOutput interface
public:
	virtual FIntPoint GetRequestedSize() const override { return UMediaOutput::RequestCaptureSourceSize; }
	virtual EPixelFormat GetRequestedPixelFormat() const override { return EPixelFormat::PF_B8G8R8A8; }
	virtual EMediaCaptureConversionOperation GetConversionOperation(EMediaCaptureSourceType InSourceType) const override { return EMediaCaptureConversionOperation::SET_ALPHA_ONE; }
protected:
	UE_API virtual UMediaCapture* CreateMediaCaptureImpl() override;
	//~ End UMediaOutput interface

public:

	UE_API void SetImageChannel(TWeakPtr<FRemoteSessionImageChannel> ImageChannel);
	TWeakPtr<FRemoteSessionImageChannel> GetImageChannel() const { return ImageChannel; }

private:

	TWeakPtr<FRemoteSessionImageChannel> ImageChannel;
};


UCLASS(MinimalAPI, BlueprintType)
class URemoteSessionMediaCapture : public UMediaCapture
{
	GENERATED_BODY()

public:

	//~ Begin UMediaCapture interface
	UE_API virtual void OnFrameCaptured_RenderingThread(const FCaptureBaseData& InBaseData, TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> InUserData, void* InBuffer, int32 Width, int32 Height, int BytesPerRow) override;
	UE_API virtual bool InitializeCapture() override;
	//~ End UMediaCapture interface

private:

	UE_API void CacheValues();

private:

	TSharedPtr<FRemoteSessionImageChannel> ImageChannel;
};

#undef UE_API
