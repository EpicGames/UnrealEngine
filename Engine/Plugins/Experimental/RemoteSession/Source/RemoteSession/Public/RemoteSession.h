// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointer.h"

#define UE_API REMOTESESSION_API

class IRemoteSessionRole;
class IRemoteSessionUnmanagedRole;
class IAnalyticsProviderET;
enum class ERemoteSessionChannelMode : int32;
enum class EAnalyticsRecordEventMode : uint8;
struct FRemoteSessionChannelInfo;


REMOTESESSION_API DECLARE_LOG_CATEGORY_EXTERN(LogRemoteSession, Log, All);

class IRemoteSessionChannelFactoryWorker;

class IRemoteSessionModule : public IModuleInterface
{
public:

	/** IModuleInterface implementation */
	virtual void StartupModule() override = 0;
	virtual void ShutdownModule() override = 0;
	/** End IModuleInterface implementation */

public:

	enum
	{
		kDefaultPort = 2049,
	};

public:

	/** Register a third party channel */
	virtual void AddChannelFactory(const FStringView InChannelName, ERemoteSessionChannelMode InHostMode, TWeakPtr<IRemoteSessionChannelFactoryWorker> Worker) = 0;

	/** Unregister a channel factory */
	virtual void RemoveChannelFactory(TWeakPtr<IRemoteSessionChannelFactoryWorker> Worker) = 0;


public:
	/** Client implementation */
	
	/** Returns a reference to the client role (if any) */
	virtual TSharedPtr<IRemoteSessionRole> CreateClient(const TCHAR* RemoteAddress) = 0;

	/** Stops the client. After this CreateClient() must be called if a new connection is desired */
	virtual void StopClient(TSharedPtr<IRemoteSessionRole> Client, const FString& InReason) = 0;

	static UE_API FString 		GetLocalVersion();

	static UE_API FString		GetLastSupportedVersion();

	/** Opaque build identifier sent in the Hello handshake; surfaced to the peer as "BuildInfo". */
	static UE_API void			SetClientBuildInfo(const FString& BuildInfo);

	static UE_API FString		GetClientBuildInfo();

	/** Analytics provider for RemoteSession events; falls back to FEngineAnalytics when unset. */
	static UE_API void			SetAnalyticsProvider(TSharedPtr<IAnalyticsProviderET> Provider);
    
public:
	/** Server implementation */

	/** Starts a RemoteSession server that listens for clients on the provided port */
	virtual void InitHost(const int16 Port=0) = 0;

	/** Returns true/false based on the running state of the host server */
	virtual bool IsHostRunning() const = 0;

	/** Returns true/false if a client is connected */
	virtual bool IsHostConnected() const = 0;

	/** Stops the server, after this InitHost() must be called if a new connection is desired */
	virtual void StopHost(const FString& InReason) = 0;

	/** Returns a reference to the server role (if any) */
	virtual TSharedPtr<IRemoteSessionRole> GetHost() const = 0;

	/** Returns host that is not controlled by the module */
	virtual TSharedPtr<IRemoteSessionUnmanagedRole> CreateHost(TArray<FRemoteSessionChannelInfo> SupportedChannels, int32 Port) const = 0;

	/** Event name for the host connection-established analytics event. Resets on StopHost. */
	virtual void SetHostAnalyticsEventName(const FString& EventName) = 0;

	/** Record mode for the host connection-established analytics event. Resets to Cached on StopHost. */
	virtual void SetHostAnalyticsEventMode(EAnalyticsRecordEventMode Mode) = 0;
};

#undef UE_API
