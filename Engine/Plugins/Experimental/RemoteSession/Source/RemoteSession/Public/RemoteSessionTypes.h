// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RemoteSessionTypes.generated.h"

class IRemoteSessionChannel;
class IRemoteSessionRole;
class IRemoteSessionUnmanagedRole;

UENUM()
enum class ERemoteSessionChannelMode : int32
{
	Unknown,
	Read,
	Write,
	MaxValue,
};

UENUM()
enum class ERemoteSessionChannelChange : int32
{
	Created,
	Destroyed
};

UENUM()
enum class ERemoteSessionConnectionChange : int32
{
	Connected,
	Disconnected
};


USTRUCT()
struct FRemoteSessionChannelInfo
{
	GENERATED_BODY()

		UPROPERTY(config)
		FString Type;

	UPROPERTY(config)
		ERemoteSessionChannelMode Mode = ERemoteSessionChannelMode::Unknown;

	FRemoteSessionChannelInfo() = default;
	FRemoteSessionChannelInfo(FString InType, ERemoteSessionChannelMode InMode)
		: Type(InType), Mode(InMode)
	{ }
};


const TCHAR* LexToString(ERemoteSessionChannelMode InMode);
void LexFromString(ERemoteSessionChannelMode& Value, const TCHAR* String);

DECLARE_MULTICAST_DELEGATE_TwoParams(FOnRemoteSessionConnectionChange, IRemoteSessionRole* /*Role*/, ERemoteSessionConnectionChange /*State*/);
DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnRemoteSessionChannelChange, IRemoteSessionRole* /*Role*/, TWeakPtr<IRemoteSessionChannel> /*Instance*/, ERemoteSessionChannelChange /*State*/);
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnRemoteSessionReceiveChannelList, IRemoteSessionRole* /*Role*/, TArrayView< FRemoteSessionChannelInfo> /*ChannelList*/);


/** Settings for the Asset Management framework, which can be used to discover, load, and audit game-specific asset types */
UCLASS(MinimalAPI, config = Engine)
class URemoteSessionSettings : public UObject
{
	GENERATED_BODY()

public:

	/* Port that the host will listen on */
	UPROPERTY(config)
	int32 HostPort = 2049;

	/* Time until a connection will timeout  */
	UPROPERTY(config)
	int32 ConnectionTimeout = 7;

	/* Time until a connection will timeout when a debugger is attached  */
	UPROPERTY(config)
	int32 ConnectionTimeoutWhenDebugging = 30;

	/* Time between pings  */
	UPROPERTY(config)
	int32 PingTime = 1;

	/* Whether RemoteSession is functional in a shipping build */
	UPROPERTY(config)
	bool bAllowInShipping = false;

	/* Does PIE automatically start hosting a session? */
	UPROPERTY(config)
	bool bAutoHostWithPIE = true;

	/* Does launching a game automatically host a session? */
	UPROPERTY(config)
	bool bAutoHostWithGame = true;

	/* Image quality (1-100) */
	UPROPERTY(config)
	int32 ImageQuality = 80;

	/* Framerate of images from the host (will be min of this value and the actual framerate of the game */
	UPROPERTY(config)
	int32 FrameRate = 60;

	/* Restrict remote session to these channels. If empty all registered channels are available */
	UPROPERTY(config)
	TArray<FString> AllowedChannels;

	/* Don't allow these channels to be used */
	UPROPERTY(config)
	TArray<FString> DeniedChannels;
};
