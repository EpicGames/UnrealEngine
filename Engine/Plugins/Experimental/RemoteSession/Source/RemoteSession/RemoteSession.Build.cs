// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using UnrealBuildTool;

public class RemoteSession : ModuleRules
{
	public RemoteSession(ReadOnlyTargetRules Target) : base(Target)
	{
		bRequiresPlatformSDK = true;
		
		PrivateIncludePaths.AddRange(
			new string[] {
			}
		);
			
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"MediaIOCore",
				"BackChannel",
				"ApplicationCore",
				"XRBase",
				// ... add other public dependencies that you statically link with here ...
			}
		);
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"Engine",
				"Slate",
				"SlateCore",
				"InputDevice",
				"InputCore",
				"RHI",
				"Renderer",
				"RenderCore",
				"ImageWrapper",
				"MovieSceneCapture",
				"Sockets",
				"EngineSettings",
				"HeadMountedDisplay",
				"AugmentedReality",
				"AnalyticsET",
				// iOS uses the Apple Image Utils plugin for GPU accellerated JPEG compression
				"AppleImageUtils"
			}
		);

		// NetworkServiceDiscovery is optional - used for mDNS host advertisement.
		// We add the include path directly (not via PrivateIncludePathModuleNames)
		// to avoid UBT enforcing a plugin dependency, since this is a soft/runtime dependency.
		{
			string NSDPublicPath = Path.Combine(PluginDirectory, "..", "NetworkServiceDiscovery", "Source", "NetworkServiceDiscovery", "Public");
			if (Directory.Exists(NSDPublicPath))
			{
				PrivateIncludePaths.Add(NSDPublicPath);
				PrivateDefinitions.Add("WITH_NETWORK_SERVICE_DISCOVERY=1");
			}
			else
			{
				PrivateDefinitions.Add("WITH_NETWORK_SERVICE_DISCOVERY=0");
			}
		}

		if (Target.bBuildEditor == true)
		{
			//reference the module "MyModule"
			PrivateDependencyModuleNames.Add("EditorFramework");
			PrivateDependencyModuleNames.Add("UnrealEd");

			//PrivateDependencyModuleNames.Add("PixelStreaming");
			//PrivateDependencyModuleNames.Add("PixelStreamingServers");
			//PrivateDependencyModuleNames.Add("PixelStreamingEditor");

			PrivateDependencyModuleNames.Add("PixelStreaming2");
			PrivateDependencyModuleNames.Add("PixelStreaming2Settings");
			PrivateDependencyModuleNames.Add("PixelStreaming2Servers");
			PrivateDependencyModuleNames.Add("PixelStreaming2Editor");
		}
	}
}
