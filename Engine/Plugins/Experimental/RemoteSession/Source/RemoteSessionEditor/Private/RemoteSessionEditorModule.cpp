// Copyright Epic Games, Inc. All Rights Reserved.

#include "RemoteSessionEditorModule.h"
#include "RemoteSessionEditorStyle.h"

#include "Modules/ModuleManager.h"
#include "Widgets/SRemoteSessionStream.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"


LLM_DEFINE_TAG(RemoteSession_RemoteSessionEditor);
#define LOCTEXT_NAMESPACE "FRemoteSessionEditorModule"

class FRemoteSessionEditorModule : public IModuleInterface
{
public:
	virtual void StartupModule() override
	{
		LLM_SCOPE_BYTAG(RemoteSession_RemoteSessionEditor);

		FRemoteSessionEditorStyle::Register();

		{
			constexpr bool bSortChildren = true;
			const IWorkspaceMenuStructure& MenuStructure = WorkspaceMenu::GetMenuStructure();
			TSharedRef<FWorkspaceItem> MenuGroup = MenuStructure.GetLevelEditorVirtualProductionCategory()->AddGroup(
				LOCTEXT("WorkspaceMenu_VirtualProductionCategory", "VirtualProduction"),
				FSlateIcon(),
				bSortChildren);

			SRemoteSessionStream::RegisterNomadTabSpawner(MenuGroup);
		}
	}

	virtual void ShutdownModule() override
	{
		LLM_SCOPE_BYTAG(RemoteSession_RemoteSessionEditor);

		if (!IsEngineExitRequested() && UObjectInitialized())
		{
			FRemoteSessionEditorStyle::Unregister();
			SRemoteSessionStream::UnregisterNomadTabSpawner();
		}
	}
};

IMPLEMENT_MODULE(FRemoteSessionEditorModule, RemoteSessionEditor)

#undef LOCTEXT_NAMESPACE
