// Copyright Epic Games, Inc. All Rights Reserved.

#include "RemoteSessionEditorStyle.h"

#include "Brushes/SlateImageBrush.h"
#include "Misc/Paths.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"

namespace RemoteSessionStyle
{
	const FVector2D Icon16x16(16.0f, 16.0f);
	const FVector2D Icon20x20(20.0f, 20.0f);
	const FVector2D Icon40x40(40.0f, 40.0f);
	const FVector2D Icon64x64(64.0f, 64.0f);

	const FName NAME_StyleName(TEXT("RemoteSessionStyle"));

	static TUniquePtr<FSlateStyleSet> StyleInstance;
}

// This is to fix the issue that SlateStyleMacros like IMAGE_BRUSH look for RootToContentDir but StyleSet->RootToContentDir is how this style is set up
#define RootToContentDir RemoteSessionStyle::StyleInstance->RootToContentDir

void FRemoteSessionEditorStyle::Register()
{
	RemoteSessionStyle::StyleInstance = MakeUnique<FSlateStyleSet>(RemoteSessionStyle::NAME_StyleName);
	RemoteSessionStyle::StyleInstance->SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));
	RemoteSessionStyle::StyleInstance->SetContentRoot(FPaths::EnginePluginsDir() / TEXT("Experimental/RemoteSession/Content/Editor/Icons/"));


	RemoteSessionStyle::StyleInstance->Set("RemoteSessionStream", new IMAGE_BRUSH_SVG("RemoteSessionStream", RemoteSessionStyle::Icon16x16));
	RemoteSessionStyle::StyleInstance->Set("TabIcons.RemoteSession.Small", new IMAGE_BRUSH("RemoteSession_Stream_16x", RemoteSessionStyle::Icon16x16));
	RemoteSessionStyle::StyleInstance->Set("RemoteSessionStream.Stream", new IMAGE_BRUSH("RemoteSession_Stream_40x", RemoteSessionStyle::Icon40x40));
	RemoteSessionStyle::StyleInstance->Set("RemoteSessionStream.Stop", new IMAGE_BRUSH("RemoteSession_Stop_40x", RemoteSessionStyle::Icon40x40));
	RemoteSessionStyle::StyleInstance->Set("RemoteSessionStream.Settings", new IMAGE_BRUSH("RemoteSession_Settings_40x", RemoteSessionStyle::Icon40x40));

	FSlateStyleRegistry::RegisterSlateStyle(*RemoteSessionStyle::StyleInstance.Get());
}

void FRemoteSessionEditorStyle::Unregister()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*RemoteSessionStyle::StyleInstance.Get());
	RemoteSessionStyle::StyleInstance.Reset();
}

FName FRemoteSessionEditorStyle::GetStyleSetName()
{
	return RemoteSessionStyle::NAME_StyleName;
}

const ISlateStyle& FRemoteSessionEditorStyle::Get()
{
	check(RemoteSessionStyle::StyleInstance.IsValid());
	return *RemoteSessionStyle::StyleInstance.Get();
}

#undef IMAGE_BRUSH
