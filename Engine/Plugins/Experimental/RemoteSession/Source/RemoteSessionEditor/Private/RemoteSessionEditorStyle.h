// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

class FName;

class FSlateStyleSet;
class ISlateStyle;

//////////////////////////////////////////////////////////////////////////
// FRemoteSessionEditorStyle

class FRemoteSessionEditorStyle
{
public:
	static void Register();
	static void Unregister();

	static FName GetStyleSetName();

	static const ISlateStyle& Get();
};
