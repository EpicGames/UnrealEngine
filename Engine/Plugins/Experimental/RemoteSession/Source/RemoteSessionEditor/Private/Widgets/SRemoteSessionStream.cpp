// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SRemoteSessionStream.h"

#include "Blueprint/UserWidget.h"
#include "Framework/Application/SlateApplication.h"
#include "Channels/RemoteSessionChannel.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "DetailsViewArgs.h"
#include "Slate/WidgetRenderer.h"
#include "Engine/Blueprint.h"
#include "Widgets/Docking/SDockTab.h"
#include "IRemoteSessionRole.h"
#include "Widgets/Images/SImage.h"
#include "SlateOptMacros.h"
#include "Widgets/Layout/SScaleBox.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/SVirtualWindow.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "EditorSupportDelegates.h"
#include "Engine/TextureRenderTarget2D.h"
#include "IDetailsView.h"
#include "LevelEditor.h"
#include "PropertyEditorModule.h"
#include "ScopedTransaction.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"

#include "RemoteSession.h"
#include "Channels/RemoteSessionImageChannel.h"
#include "Channels/RemoteSessionInputChannel.h"
#include "RemoteSession.h"
#include "ImageProviders/RemoteSessionMediaOutput.h"
#include "RemoteSessionEditorStyle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SRemoteSessionStream)

#define LOCTEXT_NAMESPACE "RemoteSessionStream"

namespace RemoteSessionStream
{
	static const FName RemoteSessionStreamApp = "SRemoteSessionStreamApp";
	static const FName LevelEditorModuleName = "LevelEditor";
	static FDelegateHandle LevelEditorTabManagerChangedHandle;

	TSharedRef<SDockTab> CreateMediaFrameworkCaptureCameraViewportTab(const FSpawnTabArgs& Args)
	{
		return SNew(SDockTab)
			.TabRole(ETabRole::NomadTab)
			[
				SNew(SRemoteSessionStream)
			];
	}
}


URemoteSessionStreamWidgetUserData::URemoteSessionStreamWidgetUserData()
	: Size(1280, 720)
	, Port(IRemoteSessionModule::kDefaultPort)
{
}


TWeakPtr<SRemoteSessionStream> SRemoteSessionStream::WidgetInstance;


void SRemoteSessionStream::RegisterNomadTabSpawner(TSharedRef<FWorkspaceItem> InWorkspaceItem)
{
	auto RegisterTabSpawner = [InWorkspaceItem]()
	{
		LLM_SCOPE_BYNAME(TEXT("RemoteSession/RemoteSessionStream"));
		FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>(RemoteSessionStream::LevelEditorModuleName);
		TSharedPtr<FTabManager> LevelEditorTabManager = LevelEditorModule.GetLevelEditorTabManager();

		LevelEditorTabManager->RegisterTabSpawner(RemoteSessionStream::RemoteSessionStreamApp, FOnSpawnTab::CreateStatic(&RemoteSessionStream::CreateMediaFrameworkCaptureCameraViewportTab))
			.SetDisplayName(LOCTEXT("TabTitle", "Remote Session Stream"))
			.SetTooltipText(LOCTEXT("TooltipText", "Stream a particular UMG to a Remote Session app."))
			.SetGroup(InWorkspaceItem)
			.SetIcon(FSlateIcon(FRemoteSessionEditorStyle::GetStyleSetName(), "RemoteSessionStream"));
	};

	FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));
	if (LevelEditorModule.GetLevelEditorTabManager())
	{
		RegisterTabSpawner();
	}
	else
	{
		RemoteSessionStream::LevelEditorTabManagerChangedHandle = LevelEditorModule.OnTabManagerChanged().AddLambda(RegisterTabSpawner);
	}
}

void SRemoteSessionStream::UnregisterNomadTabSpawner()
{
	if (FSlateApplication::IsInitialized() && FModuleManager::Get().IsModuleLoaded(RemoteSessionStream::LevelEditorModuleName))
	{
		FLevelEditorModule* LevelEditorModule = FModuleManager::GetModulePtr<FLevelEditorModule>(RemoteSessionStream::LevelEditorModuleName);
		TSharedPtr<FTabManager> LevelEditorTabManager;
		if (LevelEditorModule)
		{
			LevelEditorTabManager = LevelEditorModule->GetLevelEditorTabManager();
			LevelEditorModule->OnTabManagerChanged().Remove(RemoteSessionStream::LevelEditorTabManagerChangedHandle);
		}

		if (LevelEditorTabManager.IsValid())
		{
			LevelEditorTabManager->UnregisterTabSpawner(RemoteSessionStream::LevelEditorModuleName);
		}
	}
}

TSharedPtr<SRemoteSessionStream> SRemoteSessionStream::GetPanelInstance()
{
	return SRemoteSessionStream::WidgetInstance.Pin();
}

SRemoteSessionStream::~SRemoteSessionStream()
{
	EnabledStreaming(false);
}

void SRemoteSessionStream::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(WidgetUserData);
	Collector.AddReferencedObject(RenderTarget2D);
	Collector.AddReferencedObject(UserWidget);
	Collector.AddReferencedObject(WidgetWorld);
	Collector.AddReferencedObject(MediaOutput);
	Collector.AddReferencedObject(MediaCapture);
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SRemoteSessionStream::Construct(const FArguments& InArgs)
{
	SetCanTick(false);

	WidgetInstance = StaticCastSharedRef<SRemoteSessionStream>(AsShared());

	bIsStreaming = false;
	ResetUObject();

	WidgetUserData = NewObject<URemoteSessionStreamWidgetUserData>();

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bUpdatesFromSelection = false;
	DetailsViewArgs.bLockable = false;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.bAllowFavoriteSystem = false;
	DetailsViewArgs.bShowOptions = false;
	DetailsViewArgs.bShowPropertyMatrixButton = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.ViewIdentifier = "RemoteSessionStream";
	DetailView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
	DetailView->SetObject(WidgetUserData);

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(FMargin(2.f))
		[
			MakeToolBar()
		]
		+ SVerticalBox::Slot()
		.FillHeight(1.f)
		.Padding(FMargin(2.f))
		[
			SAssignNew(Splitter, SSplitter)
			.Orientation(GetDefault<URemoteSessionStreamSettings>()->bIsVerticalSplitterOrientation ? EOrientation::Orient_Vertical : EOrientation::Orient_Horizontal)
			+ SSplitter::Slot()
			[
				SNew(SBorder)
				.Padding(FMargin(3.f))
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				.IsEnabled_Lambda([this]() { return !IsStreaming(); })
				[
					DetailView.ToSharedRef()
				]
			]
			+ SSplitter::Slot()
			[
				SNew(SBorder)
				.Padding(FMargin(3.f))
				.BorderImage(FAppStyle::GetBrush("ToolPanel.DarkGroupBorder"))
				[
					SNew(SScaleBox)
					.Stretch(this, &SRemoteSessionStream::GetViewportStretch)
					[
						SNew(SBorder)
						.Padding(FMargin(0.f))
						.BorderImage(this, &SRemoteSessionStream::GetImageBorderImage)
						[
							SNew(SImage)
							.Image(&RenderTargetBrush)
						]
					]
				]
			]
		]
	];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

const FSlateBrush* SRemoteSessionStream::GetImageBorderImage() const
{
	if (GetDefault<URemoteSessionStreamSettings>()->bShowCheckered)
	{
		return FCoreStyle::Get().GetBrush("ColorPicker.AlphaBackground");
	}
	return FAppStyle::GetBrush("ToolPanel.GroupBorder");
}

EStretch::Type SRemoteSessionStream::GetViewportStretch() const
{
	if (GetDefault<URemoteSessionStreamSettings>()->bScaleToFit)
	{
		return EStretch::ScaleToFit;
	}
	return EStretch::ScaleToFill;
}

TSharedRef<class SWidget> SRemoteSessionStream::MakeToolBar()
{
	FToolBarBuilder ToolBarBuilder(TSharedPtr<FUICommandList>(), FMultiBoxCustomization::None);
	ToolBarBuilder.BeginSection(TEXT("Stream"));
	{
		ToolBarBuilder.AddToolBarButton(
			FUIAction(
				FExecuteAction::CreateLambda([this]
				{
					EnabledStreaming(true);
				}),
				FCanExecuteAction::CreateLambda([this]
				{
					return CanStream() && !IsStreaming();
				})),
			NAME_None,
			LOCTEXT("Stream_Label", "Stream"),
			LOCTEXT("Stream_ToolTip", "Stream the target to the Remote Session."),
			FSlateIcon(FRemoteSessionEditorStyle::GetStyleSetName(), "RemoteSessionStream.Stream")
			);
		ToolBarBuilder.AddToolBarButton(
			FUIAction(
				FExecuteAction::CreateLambda([this]
				{
					EnabledStreaming(false);
				}),
				FCanExecuteAction::CreateLambda([this]
				{
					return IsStreaming();
				})
			),
			NAME_None,
			LOCTEXT("Stop_Label", "Stop"),
			LOCTEXT("Stop_ToolTip", "Stop the streaming to the Remote Session."),
			FSlateIcon(FRemoteSessionEditorStyle::GetStyleSetName(), "RemoteSessionStream.Stop")
			);
	}
	ToolBarBuilder.EndSection();

	ToolBarBuilder.BeginSection("Options");
	{
		ToolBarBuilder.AddComboButton(
			FUIAction(),
			FOnGetContent::CreateRaw(this, &SRemoteSessionStream::CreateSettingsMenu),
			LOCTEXT("Settings_Label", "Settings"),
			LOCTEXT("Settings_ToolTip", "Settings"),
			FSlateIcon(FRemoteSessionEditorStyle::GetStyleSetName(), "RemoteSessionStream.Settings")
		);
	}
	ToolBarBuilder.EndSection();


	return ToolBarBuilder.MakeWidget();
}

TSharedRef<SWidget> SRemoteSessionStream::CreateSettingsMenu()
{
	FMenuBuilder SettingsMenuBuilder(true, nullptr);

	SettingsMenuBuilder.AddMenuEntry(
		LOCTEXT("SplitterOrientation_Label", "Vertical Split"),
		LOCTEXT("SplitterOrientation_Tooltip", "Split the captures vertically or horizontally."),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([this]
			{
				const FScopedTransaction Transaction(LOCTEXT("SplitterOrientation_Transaction", "Capture Splitter Orientation"));
				GetMutableDefault<URemoteSessionStreamSettings>()->Modify();
				GetMutableDefault<URemoteSessionStreamSettings>()->bIsVerticalSplitterOrientation = !GetDefault<URemoteSessionStreamSettings>()->bIsVerticalSplitterOrientation;
				Splitter->SetOrientation(GetDefault<URemoteSessionStreamSettings>()->bIsVerticalSplitterOrientation ? EOrientation::Orient_Vertical : EOrientation::Orient_Horizontal);
				GetMutableDefault<URemoteSessionStreamSettings>()->SaveConfig();
			}),
			FCanExecuteAction(),
			FIsActionChecked::CreateLambda([this]
			{
				return GetDefault<URemoteSessionStreamSettings>()->bIsVerticalSplitterOrientation;
			})
		),
		NAME_None,
		EUserInterfaceActionType::ToggleButton
		);

	SettingsMenuBuilder.BeginSection("ViewportSection", LOCTEXT("ViewportSectionHeader", "Viewport Options"));
	{
		SettingsMenuBuilder.AddSubMenu(
			LOCTEXT("Background", "Background"),
			LOCTEXT("BackgroundTooltip", "Set the viewport's background."),
			FNewMenuDelegate::CreateSP(this, &SRemoteSessionStream::GenerateBackgroundMenuContent)
		);

		SettingsMenuBuilder.AddMenuEntry(
			LOCTEXT("ScaleToFit_Label", "Scale To Fit"),
			LOCTEXT("ScaleToFit_Tooltip", "The UMG will be scaled to fit the viewport in the editor."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this]
				{
					const FScopedTransaction Transaction(LOCTEXT("ScaleToFit_Transaction", "Scale To Fit"));
					GetMutableDefault<URemoteSessionStreamSettings>()->Modify();
					GetMutableDefault<URemoteSessionStreamSettings>()->bScaleToFit = !GetDefault<URemoteSessionStreamSettings>()->bScaleToFit;
					GetMutableDefault<URemoteSessionStreamSettings>()->SaveConfig();
				}),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this]
				{
					return GetDefault<URemoteSessionStreamSettings>()->bScaleToFit;
				})
			),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
			);
	}
	SettingsMenuBuilder.EndSection();

	return SettingsMenuBuilder.MakeWidget();
}

void SRemoteSessionStream::GenerateBackgroundMenuContent(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.AddMenuEntry(
		LOCTEXT("Checkered_Label", "Checkered"),
		LOCTEXT("Checkered_Tooltip", "Checkered background pattern behind the texture."),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([this]
			{
				const FScopedTransaction Transaction(LOCTEXT("Checkered_Transaction", "Checkered"));
				GetMutableDefault<URemoteSessionStreamSettings>()->Modify();
				GetMutableDefault<URemoteSessionStreamSettings>()->bShowCheckered = true;
				GetMutableDefault<URemoteSessionStreamSettings>()->SaveConfig();
			}),
			FCanExecuteAction(),
			FIsActionChecked::CreateLambda([this]
			{
				return GetDefault<URemoteSessionStreamSettings>()->bShowCheckered;
			})
		),
		NAME_None,
		EUserInterfaceActionType::RadioButton
		);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("SolidBackground_Label", "Solid Color"),
		LOCTEXT("SolidBackground_Tooltip", "Solid color background."),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([this]
			{
				const FScopedTransaction Transaction(LOCTEXT("SolidBackground_Transaction", "Solid Color"));
				GetMutableDefault<URemoteSessionStreamSettings>()->Modify();
				GetMutableDefault<URemoteSessionStreamSettings>()->bShowCheckered = false;
				GetMutableDefault<URemoteSessionStreamSettings>()->SaveConfig();
			}),
			FCanExecuteAction(),
			FIsActionChecked::CreateLambda([this]
			{
				return !GetDefault<URemoteSessionStreamSettings>()->bShowCheckered;
			})
		),
		NAME_None,
		EUserInterfaceActionType::RadioButton
		);
}

void SRemoteSessionStream::Tick(float InDeltaTime)
{
	if (RemoteSessionHost)
	{
		RemoteSessionHost->Tick(InDeltaTime);
	}

	if (UserWidget && !UserWidget->IsDesignTime() && WidgetRenderer && RenderTarget2D && VirtualWindow)
	{
		WidgetRenderer->DrawWindow(RenderTarget2D, VirtualWindow->GetHittestGrid(), VirtualWindow.ToSharedRef(), 1.f, WidgetSize, InDeltaTime);
	}
}

void SRemoteSessionStream::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	// Call super. Mac is complaining at compilation because we override another tick function.
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
}

TStatId SRemoteSessionStream::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(SRemoteSessionStream, STATGROUP_Tickables);
}

bool SRemoteSessionStream::CanStream() const
{
	return WidgetUserData && WidgetUserData->WidgetClass.Get() && WidgetUserData->Size.X > 1 && WidgetUserData->Size.Y > 1;
}

void SRemoteSessionStream::EnabledStreaming(bool bInStreaming)
{
	if (bInStreaming != bIsStreaming)
	{
		if (bInStreaming && CanStream())
		{
			// Cache values
			WidgetSize = WidgetUserData->Size;

			// Create Widget
			{
				WidgetWorld = GEditor->GetEditorWorldContext().World();
				if (WidgetWorld)
				{
					check(WidgetUserData->WidgetClass);
					UserWidget = CreateWidget<UUserWidget>(WidgetWorld, WidgetUserData->WidgetClass);
				}
			}

			if (UserWidget == nullptr)
			{
				EnabledStreaming(false);
				return;
			}

			// Create Host
			if (IRemoteSessionModule* RemoteSession = FModuleManager::LoadModulePtr<IRemoteSessionModule>("RemoteSession"))
			{
				TArray<FRemoteSessionChannelInfo> SupportedChannels;
				SupportedChannels.Emplace(FRemoteSessionInputChannel::StaticType(), ERemoteSessionChannelMode::Read);
				SupportedChannels.Emplace(FRemoteSessionImageChannel::StaticType(), ERemoteSessionChannelMode::Write);
				RemoteSessionHost = RemoteSession->CreateHost(MoveTemp(SupportedChannels), WidgetUserData->Port);
				RemoteSessionHost->RegisterChannelChangeDelegate(FOnRemoteSessionChannelChange::FDelegate::CreateSP(this, &SRemoteSessionStream::OnRemoteSessionChannelChange));
				RemoteSessionHost->Tick(0.f);
			}

			// Create Output. Will only be activated once a client is connected
			MediaOutput = NewObject<URemoteSessionMediaOutput>();

			RenderTarget2D = WidgetUserData->RenderTarget;
			if (RenderTarget2D == nullptr)
			{
				RenderTarget2D = NewObject<UTextureRenderTarget2D>();
			}
			RenderTarget2D->ClearColor = FLinearColor::Transparent;
			bool bInForceLinearGamma = false;
			RenderTarget2D->InitCustomFormat(WidgetSize.X, WidgetSize.Y, EPixelFormat::PF_B8G8R8A8, bInForceLinearGamma);

			// Update UI
			RenderTargetBrush.ImageSize = WidgetSize;
			RenderTargetBrush.SetResourceObject(RenderTarget2D);

			// Create renderer
			VirtualWindow = SNew(SVirtualWindow).Size(WidgetSize);
			VirtualWindow->SetContent(UserWidget->TakeWidget());
			VirtualWindow->Resize(WidgetSize);

			if (FSlateApplication::IsInitialized())
			{
				FSlateApplication::Get().RegisterVirtualWindow(VirtualWindow.ToSharedRef());
			}

			bool bApplyGammaCorrection = false;
			WidgetRenderer = new FWidgetRenderer(bApplyGammaCorrection);
			WidgetRenderer->DrawWindow(RenderTarget2D, VirtualWindow->GetHittestGrid(), VirtualWindow.ToSharedRef(), 1.f, WidgetSize, 0.1f);

			// Register callback
			GEditor->OnBlueprintPreCompile().AddSP(this, &SRemoteSessionStream::OnBlueprintPreCompile);
			FEditorSupportDelegates::PrepareToCleanseEditorObject.AddRaw(this, &SRemoteSessionStream::OnPrepareToCleanseEditorObject);
			FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
			AssetRegistryModule.Get().OnAssetRemoved().AddRaw(this, &SRemoteSessionStream::HandleAssetRemoved);
			FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));
			LevelEditorModule.OnMapChanged().AddRaw(this, &SRemoteSessionStream::OnMapChanged);
			FEditorDelegates::OnAssetsCanDelete.AddRaw(this, &SRemoteSessionStream::CanDeleteAssets);
		}
		else
		{
			bInStreaming = false;

			FEditorDelegates::OnAssetsCanDelete.RemoveAll(this);
			if (FLevelEditorModule* LevelEditorModule = FModuleManager::GetModulePtr<FLevelEditorModule>(TEXT("LevelEditor")))
			{
				LevelEditorModule->OnMapChanged().RemoveAll(this);
			}
			if (FAssetRegistryModule* AssetRegistryModule = FModuleManager::GetModulePtr<FAssetRegistryModule>(TEXT("AssetRegistry")))
			{
				IAssetRegistry* AssetRegistry = AssetRegistryModule->TryGet();
				if (AssetRegistry)
				{
					AssetRegistry->OnAssetRemoved().RemoveAll(this);
				}
			}
			FEditorSupportDelegates::PrepareToCleanseEditorObject.RemoveAll(this);
			GEditor->OnBlueprintPreCompile().RemoveAll(this);

			if (VirtualWindow.IsValid() && FSlateApplication::IsInitialized())
			{
				FSlateApplication::Get().UnregisterVirtualWindow(VirtualWindow.ToSharedRef());
			}

			RenderTargetBrush.SetResourceObject(nullptr);
			if (WidgetRenderer)
			{
				BeginCleanup(WidgetRenderer);
				WidgetRenderer = nullptr;
			}
			VirtualWindow.Reset();
			ResetUObject();
			RemoteSessionHost.Reset();
		}
	}

	bIsStreaming = bInStreaming;
}

void SRemoteSessionStream::ResetUObject()
{
	//WidgetUserData stay valid
	RenderTarget2D = nullptr;
	UserWidget = nullptr;
	WidgetWorld = nullptr;
	MediaOutput = nullptr;
	if (MediaCapture)
	{
		MediaCapture->StopCapture(false);
		MediaCapture = nullptr;
	}
}

void SRemoteSessionStream::OnRemoteSessionChannelChange(IRemoteSessionRole* Role, TWeakPtr<IRemoteSessionChannel> Channel, ERemoteSessionChannelChange Change)
{
	TSharedPtr<IRemoteSessionChannel> Pinned = Channel.Pin();

	if (Pinned && Change == ERemoteSessionChannelChange::Created)
	{
		if (Pinned->GetType() == FRemoteSessionInputChannel::StaticType())
		{
			OnInputChannelCreated(Pinned);
		}
		else if (Pinned->GetType() == FRemoteSessionImageChannel::StaticType())
		{
			OnImageChannelCreated(Pinned);
		}
	}
}

void SRemoteSessionStream::OnInputChannelCreated(TWeakPtr<IRemoteSessionChannel> Channel)
{
	TSharedPtr<FRemoteSessionInputChannel> InputChannel = StaticCastSharedPtr<FRemoteSessionInputChannel>(Channel.Pin());
	if (InputChannel)
	{
		InputChannel->SetPlaybackWindow(VirtualWindow, nullptr);
		InputChannel->TryRouteTouchMessageToWidget(true);
	}
}

void SRemoteSessionStream::OnImageChannelCreated(TWeakPtr<IRemoteSessionChannel> Channel)
{
	TSharedPtr<FRemoteSessionImageChannel> ImageChannel = StaticCastSharedPtr<FRemoteSessionImageChannel>(Channel.Pin());
	if (ImageChannel)
	{
		ImageChannel->SetImageProvider(nullptr);
		MediaOutput->SetImageChannel(ImageChannel);
		MediaCapture = Cast<URemoteSessionMediaCapture>(MediaOutput->CreateMediaCapture());
		MediaCapture->CaptureTextureRenderTarget2D(RenderTarget2D, FMediaCaptureOptions());
	}
}

void SRemoteSessionStream::OnBlueprintPreCompile(UBlueprint* Blueprint)
{
	if (Blueprint && UserWidget && Blueprint->GeneratedClass == UserWidget->GetClass())
	{
		EnabledStreaming(false);
	}
}

void SRemoteSessionStream::OnPrepareToCleanseEditorObject(UObject* Object)
{
	if (Object == RenderTarget2D || Object == UserWidget || Object == WidgetWorld || (UserWidget && Object == UserWidget->GetClass()) || Object == MediaCapture)
	{
		EnabledStreaming(false);
	}
}

void SRemoteSessionStream::HandleAssetRemoved(const FAssetData& AssetData)
{
	if (FAssetData(RenderTarget2D) == AssetData || (UserWidget && AssetData.GetPackage() == UserWidget->GetClass()->GetOutermost()))
	{
		EnabledStreaming(false);
	}
}

void SRemoteSessionStream::OnMapChanged(UWorld* World, EMapChangeType ChangeType)
{
	if (World == WidgetWorld && ChangeType == EMapChangeType::TearDownWorld)
	{
		EnabledStreaming(false);
	}
}

void SRemoteSessionStream::CanDeleteAssets(const TArray<UObject*>& InAssetsToDelete, FCanDeleteAssetResult& CanDeleteResult)
{
	if (UserWidget)
	{
		for (UObject* Obj : InAssetsToDelete)
		{
			if (UserWidget->GetClass()->GetOutermost() == Obj->GetOutermost())
			{
				UE_LOGF(LogRemoteSession, Warning, "Asset '%ls' can't be deleted because it is currently used by the Remote Session Stream.", *Obj->GetPathName());
				CanDeleteResult.Set(false);
				break;
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE

