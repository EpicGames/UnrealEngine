// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Engine/AssetUserData.h"
#include "Styling/SlateBrush.h"
#include "TickableEditorObject.h"
#include "Templates/SubclassOf.h"
#include "Widgets/SCompoundWidget.h"
#include "UObject/GCObject.h"
#include "SRemoteSessionStream.generated.h"

class IRemoteSessionChannel;
class IRemoteSessionRole;
class IRemoteSessionUnmanagedRole;
enum class ERemoteSessionChannelChange : int32;
namespace EStretch { enum Type : int; }

struct FAssetData;
struct FCanDeleteAssetResult;
class FMenuBuilder;
class FWidgetRenderer;
class IDetailsView;
class SImage;
class SSplitter;
class SVirtualWindow;
class UBlueprint;
class URemoteSessionMediaOutput;
class URemoteSessionMediaCapture;
class UTextureRenderTarget2D;
class UUserWidget;
class UWidgetBlueprint;
class UWorld;
class FWorkspaceItem;

enum class EMapChangeType : uint8;

/**
 * Settings for the remote session stream tab.
 */
UCLASS(MinimalAPI, config = EditorPerProjectUserSettings)
class URemoteSessionStreamSettings : public UObject
{
	GENERATED_BODY()
public:
	UPROPERTY(config)
	bool bIsVerticalSplitterOrientation = true;

	UPROPERTY(config)
	bool bShowCheckered = false;

	UPROPERTY(config)
	bool bScaleToFit = true;
};


/**
 * URemoteSessionStreamWidgetUserData
 */
UCLASS(MinimalAPI)
class URemoteSessionStreamWidgetUserData : public UAssetUserData
{
	GENERATED_BODY()

public:
	URemoteSessionStreamWidgetUserData();

	UPROPERTY(EditAnywhere, Category = "Remote Session")
	TSubclassOf<UUserWidget> WidgetClass;

	UPROPERTY(EditAnywhere, Category = "Remote Session")
	FVector2D Size;

	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Remote Session")
	TObjectPtr<UTextureRenderTarget2D> RenderTarget;

	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Remote Session")
	int16 Port;
};


/*
 * SRemoteSessionStream
 */
class SRemoteSessionStream : public SCompoundWidget, FGCObject, FTickableEditorObject
{
public:
	static void RegisterNomadTabSpawner(TSharedRef<FWorkspaceItem> InWorkspaceItem);
	static void UnregisterNomadTabSpawner();
	static TSharedPtr<SRemoteSessionStream> GetPanelInstance();

private:
	static TWeakPtr<SRemoteSessionStream> WidgetInstance;

public:
	SLATE_BEGIN_ARGS(SRemoteSessionStream){}
	SLATE_END_ARGS()

	~SRemoteSessionStream();
	void Construct(const FArguments& InArgs);
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	//~ Begin FGCObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("SRemoteSessionStream");
	}
	//~ End FGCObject interface

	//~ Begin FTickableEditorObject interface
	virtual void Tick(float DeltaTime) override;
	virtual ETickableTickType GetTickableTickType() const override { return ETickableTickType::Conditional; }
	virtual bool IsTickable() const override { return IsStreaming(); }
	virtual TStatId GetStatId() const override;
	//~ End FTickableEditorObject interface

	bool IsStreaming() const { return bIsStreaming; }
	bool CanStream() const;
	void EnabledStreaming(bool bStreaming);

private:
	TSharedRef<class SWidget> MakeToolBar();
	TSharedRef<SWidget> CreateSettingsMenu();
	void GenerateBackgroundMenuContent(FMenuBuilder& MenuBuilder);
	const FSlateBrush* GetImageBorderImage() const;
	EStretch::Type GetViewportStretch() const;

	void ResetUObject();

	void OnRemoteSessionChannelChange(IRemoteSessionRole* Role, TWeakPtr<IRemoteSessionChannel> Channel, ERemoteSessionChannelChange Change);
	void OnImageChannelCreated(TWeakPtr<IRemoteSessionChannel> Instance);
	void OnInputChannelCreated(TWeakPtr<IRemoteSessionChannel> Instance);

	void OnBlueprintPreCompile(UBlueprint* Blueprint);
	void OnPrepareToCleanseEditorObject(UObject* Object);
	void HandleAssetRemoved(const FAssetData& AssetData);
	void OnMapChanged(UWorld* World, EMapChangeType ChangeType);
	void CanDeleteAssets(const TArray<UObject*>& InAssetsToDelete, FCanDeleteAssetResult& CanDeleteResult);

private:
	TSharedPtr<IDetailsView> DetailView;
	TSharedPtr<SSplitter> Splitter;
	//TSharedPtr<SImage> RenderTargetImage;
	FSlateBrush RenderTargetBrush;

	//~ Begin GC by AddReferencedObjects
	TObjectPtr<URemoteSessionStreamWidgetUserData> WidgetUserData;
	TObjectPtr<UTextureRenderTarget2D> RenderTarget2D;
	TObjectPtr<UUserWidget> UserWidget;
	TObjectPtr<UWorld> WidgetWorld;
	TObjectPtr<URemoteSessionMediaOutput> MediaOutput;
	TObjectPtr<URemoteSessionMediaCapture> MediaCapture;
	//~ End GC by AddReferencedObjects

	TSharedPtr<IRemoteSessionUnmanagedRole> RemoteSessionHost;
	TSharedPtr<SVirtualWindow> VirtualWindow;
	FWidgetRenderer* WidgetRenderer;
	FVector2D WidgetSize;

	bool bIsStreaming;
};
