// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class RemoteSessionEditor : ModuleRules
{
	public RemoteSessionEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"RemoteSession",
				"FieldNotification",
				"UMG",
				"UMGEditor",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"AssetRegistry",
				"EditorFramework",
				"Engine",
				"LevelEditor",
				"MediaIOCore",
				"RenderCore",
				"Slate",
				"SlateCore",
				"UnrealEd",
				"WorkspaceMenuStructure",
			}
		);
	}
}
